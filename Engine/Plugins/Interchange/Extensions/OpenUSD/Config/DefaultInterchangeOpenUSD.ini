[CoreRedirects]
+ClassRedirects=(OldName="/Script/InterchangePipelines.InterchangeUsdPipeline",NewName="/Script/InterchangeOpenUSDImport.InterchangeUsdPipeline")
