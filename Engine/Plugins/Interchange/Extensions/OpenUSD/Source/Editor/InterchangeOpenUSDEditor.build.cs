// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeOpenUSDEditor : ModuleRules
	{
		public InterchangeOpenUSDEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",			// For EKeys
					"InterchangeEditor",
					"InterchangeEngine",	// For UInterchangeManager::GetInterchangeManager
					"InterchangeOpenUSDImport",
					"Slate",
					"SlateCore",
					"UnrealEd",				// For FScopedTransaction
					"UnrealUSDWrapper",		// UnrealIdentifiers in the USD translator settings customization
					"USDClasses"			// For USDProjectSettings and render context
				}
			);
		}
	}
}