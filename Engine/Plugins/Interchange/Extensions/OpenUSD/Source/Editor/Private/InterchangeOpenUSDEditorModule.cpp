// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeOpenUSDEditorModule.h"

#include "Engine/Engine.h"
#include "InterchangeManager.h"
#include "InterchangeUsdTranslatorSettingsCustomization.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

void FInterchangeOpenUSDEditorModule::StartupModule()
{
	using namespace UE::Interchange;
	auto RegisterItems = []()
	{
		// Translator settings customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(
			TEXT("InterchangeUsdTranslatorSettings"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeUsdTranslatorSettingsCustomization::MakeInstance)
		);
	};
	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}
}
void FInterchangeOpenUSDEditorModule::ShutdownModule()
{
	//Translator settings customizations
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(TEXT("PropertyEditor")))
	{
		PropertyModule->UnregisterCustomClassLayout(TEXT("InterchangeUsdTranslatorSettings"));
	}
}

IMPLEMENT_MODULE(FInterchangeOpenUSDEditorModule, InterchangeOpenUSDEditor)