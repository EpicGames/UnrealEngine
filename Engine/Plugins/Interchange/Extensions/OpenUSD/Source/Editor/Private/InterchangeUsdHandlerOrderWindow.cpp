// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUsdHandlerOrderWindow.h"

#include "InterchangeUsdTranslator.h"
#include "SchemaHandlers/SchemaHandlerEntry.h"
#include "SchemaHandlers/SchemaHandlerRegistry.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "UnrealUSDWrapper.h"

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "InterchangeUsdHandlerOrderWindow"

namespace UE::UsdHandlerOrderWindow::Private
{
	using namespace UE::Interchange::USD;

	class SSchemaHandlerRow;

	class FSchemaHandlerEntryDragDropOp : public FDragDropOperation
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FSchemaHandlerEntryDragDropOp, FDragDropOperation)
		DECLARE_DELEGATE(FOnDrop);

		FOnDrop OnDropEvent;
		TSharedPtr<SSchemaHandlerRow> DraggedRow;

		FSchemaHandlerEntryDragDropOp()
		{
			MouseCursor = EMouseCursor::GrabHandClosed;
		}

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNullWidget::NullWidget;
		}

		virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override
		{
			FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
			OnDropEvent.ExecuteIfBound();
		}
	};

	// Swaps the empty string for 'universal' if it exists in the list
	TArray<FString> GetUserFacingRenderContexts(TArray<FString> RenderContexts)
	{
		for (FString& RenderContext : RenderContexts)
		{
			if (RenderContext.IsEmpty())
			{
				RenderContext = UnrealIdentifiers::UniversalRenderContextDisplayString;
			}
		}

		return RenderContexts;
	}

	// Swaps the 'universal' string for the empty string, if it exists in the list
	TArray<FString> GetSDKFacingRenderContexts(TArray<FString> RenderContexts)
	{
		for (FString& RenderContext : RenderContexts)
		{
			if (RenderContext == UnrealIdentifiers::UniversalRenderContextDisplayString)
			{
				RenderContext = FString{};
			}
		}

		return RenderContexts;
	}

	class SSchemaHandlerRow : public STableRow<TSharedPtr<FSchemaHandlerEntry>>
	{
	public:
		typedef typename STableRow<TSharedPtr<FSchemaHandlerEntry>>::FOnCanAcceptDrop FUsdOnCanAcceptDrop;
		typedef typename STableRow<TSharedPtr<FSchemaHandlerEntry>>::FOnAcceptDrop FUsdOnAcceptDrop;

		SLATE_BEGIN_ARGS(SSchemaHandlerRow)
		{
		}
		SLATE_EVENT(FUsdOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FUsdOnAcceptDrop, OnAcceptDrop)
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			TSharedPtr<FSchemaHandlerEntry> InTreeItem,
			const TSharedRef<STableViewBase>& OwnerTable
		)
		{
			STableRow::Construct(
				STableRow::FArguments()
					.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
					.OnAcceptDrop(InArgs._OnAcceptDrop)
					.OnDragDetected(this, &SSchemaHandlerRow::OnDragDetected)
					.Padding(FMargin(4.0f, 4.0f, 4.0f, 0.0f)), // Spacing between rows, as ListView doesn't provide this
				OwnerTable
			);

			TreeItem = InTreeItem;

			TSharedPtr<SWidget> RenderContextWidget = SNullWidget::NullWidget;
			if (TreeItem && TreeItem->SchemaName == TEXT("Material"))
			{
				if (InTreeItem->bAllowCustomRenderContexts)
				{
					// For handlers that allow custom render contexts, we'll let people type them in manually
					RenderContextWidget = SNew(SEditableTextBox)
					.Cursor(EMouseCursor::Default)
					.ToolTipText(LOCTEXT("EditableRenderContextToolTip", "Comma-separated list of render contexts to try parsing, in order of priority (first item of the list will be checked first)\nIn this dialog, use 'universal' for the USD universal render context."))
					.MinDesiredWidth(70.0f)
					.Text_Lambda([this]() -> FText
					{
						if (TreeItem)
						{
							return FText::FromString(FString::Join(GetUserFacingRenderContexts(TreeItem->CustomRenderContexts), TEXT(", ")));
						}

						return {};
					})
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
					{
						if (!TreeItem || CommitType != ETextCommit::OnEnter)
						{
							return;
						}

						FString NewString = NewText.ToString();

						TArray<FString> RenderContexts;
						NewString.ParseIntoArray(RenderContexts, TEXT(","));

						for (int32 Index = RenderContexts.Num() - 1; Index >= 0; --Index)
						{
							FString& RenderContext = RenderContexts[Index];
							RenderContext.TrimStartAndEndInline();
						}

						TreeItem->CustomRenderContexts = GetSDKFacingRenderContexts(RenderContexts);
					});
				}
				else
				{
					// For other handlers, just display the default render contexts as text
					RenderContextWidget = SNew(STextBlock)
						.Text(FText::FromString(FString::Join(GetUserFacingRenderContexts(TreeItem->DefaultRenderContexts), TEXT(", "))))
						.ToolTipText(LOCTEXT("RenderContextToolTip", "List of render contexts the handler will try parsing, in order of priority (first item of the list will be checked first)\nIn this dialog 'universal' means the USD universal render context."))
						.MinDesiredWidth(50.0f)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
				}
			}

			SetRowContent(
				SNew(SBorder)
				.BorderImage_Lambda([this]()
				{
					return bBeingDragged
						? FAppStyle::GetBrush("Brushes.Recessed")
						: IsHovered()
							? FAppStyle::GetBrush("Brushes.Secondary")
							: FAppStyle::GetBrush("ToolPanel.GroupBorder");
				})
				.ColorAndOpacity_Lambda([this]()
				{
					return (bBeingDragged || !TreeItem->bEnabled)
						? FLinearColor{0.5f, 0.5f, 0.5f, 0.5f}
						: FLinearColor{1.0f, 1.0f, 1.0f, 1.0f};
				})
				.Cursor_Lambda([this]()
				{
					return IsHovered() ? EMouseCursor::GrabHand : EMouseCursor::Default;
				})
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(10.0f))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(0, 0, 10.0f, 0))
					.MaxWidth(20)
					.AutoWidth()
					[
						SNew(SCheckBox)
						.ToolTipText(LOCTEXT("CheckBoxToolTip", "Enable or disable the schema handler"))
						.Cursor(EMouseCursor::Default)
						.IsChecked_Lambda([InTreeItem]()
						{
							if (InTreeItem)
							{
								return InTreeItem->bEnabled
									? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
	
							return ECheckBoxState::Undetermined;
						})
						.OnCheckStateChanged_Lambda([InTreeItem](ECheckBoxState State)
						{
							if (InTreeItem)
							{
								if (State == ECheckBoxState::Checked)
								{
									InTreeItem->bEnabled = true;
								}
								else if (State == ECheckBoxState::Unchecked)
								{
									InTreeItem->bEnabled = false;
								}
							}
						})
					]
	
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TreeItem->HandlerName))
						.ToolTipText(LOCTEXT("HandlerName", "Handler name"))
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.FillWidth(0.7)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TreeItem->SchemaName))
						.ToolTipText(LOCTEXT("HandlerSchema", "Prim type name the handler applies to (\"IsA schema\").\nIn practice the handler may also filter for additional criteria like API schemas."))
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.FillWidth(0.7)
					[
						RenderContextWidget.ToSharedRef()
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.MaxWidth(30.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.DragHandle"))
					]
				]
			);
		}

		FReply OnDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
		{
			TSharedRef<FSchemaHandlerEntryDragDropOp> Op = MakeShared<FSchemaHandlerEntryDragDropOp>();
			Op->DraggedRow = SharedThis(this);

			bBeingDragged = true;
			Op->OnDropEvent.BindSPLambda(this, [this]()
			{
				bBeingDragged = false;
			});

			return FReply::Handled().BeginDragDrop(Op);
		}

		// We have to override this because the default table row implementation will only initiate drag and drop
		// if the list selection mode is not None, but we don't really want to allow selection on our list view
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			TSharedRef<ITypedTableView<TSharedPtr<FSchemaHandlerEntry>>> OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

			if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				return FReply::Handled()
					.DetectDrag(SharedThis(this), EKeys::LeftMouseButton)
					.SetUserFocus(OwnerTable->AsWidget(), EFocusCause::Mouse)
					.CaptureMouse(SharedThis(this));
			}

			return FReply::Unhandled();
		}

	public:
		bool bBeingDragged = false;
		TSharedPtr<FSchemaHandlerEntry> TreeItem;
	};

	TArray<TSharedPtr<FSchemaHandlerEntry>> WrapElementsWithSharedPtr(const TArray<FSchemaHandlerEntry>& Entries)
	{
		TArray<TSharedPtr<FSchemaHandlerEntry>> ResultEntries;
		ResultEntries.Reserve(Entries.Num());
		for (const FSchemaHandlerEntry& Entry : Entries)
		{
			ResultEntries.Add(MakeShared<FSchemaHandlerEntry>(Entry));
		}
		return ResultEntries;
	}

	TArray<FSchemaHandlerEntry> UnwrapElementsWithSharedPtr(const TArray<TSharedPtr<FSchemaHandlerEntry>>& Entries)
	{
		TArray<FSchemaHandlerEntry> ResultEntries;
		ResultEntries.Reserve(Entries.Num());
		for (const TSharedPtr<FSchemaHandlerEntry>& Entry : Entries)
		{
			if (Entry)
			{
				ResultEntries.Add(*Entry);
			}
		}
		return ResultEntries;
	}

	void WriteHandlerEntriesToSettings(TArray<TSharedPtr<FSchemaHandlerEntry>> Entries, UInterchangeUsdTranslatorSettings* Settings)
	{
		if (!Settings)
		{
			return;
		}

		static FProperty* CustomEntriesProperty = UInterchangeUsdTranslatorSettings::StaticClass()->FindPropertyByName(
			GET_MEMBER_NAME_CHECKED(UInterchangeUsdTranslatorSettings, CustomHandlerEntries)
		);

		FPropertyChangedEvent PropertyChangeEvent{CustomEntriesProperty};
		Settings->Modify();
		{
			Settings->CustomHandlerEntries = UnwrapElementsWithSharedPtr(Entries);
		}
		Settings->PostEditChangeProperty(PropertyChangeEvent);
	}

	TArray<TSharedPtr<FSchemaHandlerEntry>> ReadHandlerEntriesFromSettings(UInterchangeUsdTranslatorSettings* Settings)
	{
		if (!Settings)
		{
			return {};
		}

		// Try applying the existing reordering saved on the translator settings, if any
		if (Settings->CustomHandlerEntries.Num() > 0)
		{
			return WrapElementsWithSharedPtr(Settings->CustomHandlerEntries);
		}

#if USE_USD_SDK
		return WrapElementsWithSharedPtr(FSchemaHandlerRegistry::RegisteredHandlerEntries);
#else
		return {};
#endif	  // USE_USD_SDK
	}
}	 // namespace UE::UsdHandlerOrderWindow::Private

bool SInterchangeUsdHandlerOrderWindow::ShowWindow(UInterchangeUsdTranslatorSettings* InSettings)
{
	// Reference: SUsdOptionsWindow::ShowOptions

	using namespace UE::UsdHandlerOrderWindow::Private;

	if (!InSettings)
	{
		return false;
	}

	const static FString ConfigSectionName = TEXT("InterchangeUsdHandlerOrderWindow");
	const static FString WidthConfigName = TEXT("DialogWidth");
	const static FString HeightConfigName = TEXT("DialogHeight");

	// Grab our desired window size from persistent storage, if we have any
	int32 Width = 650;
	int32 Height = 850;
	{
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);

		if (GConfig->GetInt(*ConfigSectionName, *WidthConfigName, Width, GEditorPerProjectIni))
		{
			// Just to make sure we don't load something unusable
			Width = FMath::Min(FMath::Max(Width, 100), DisplayMetrics.PrimaryDisplayWidth);
		}

		if (GConfig->GetInt(*ConfigSectionName, *HeightConfigName, Height, GEditorPerProjectIni))
		{
			Height = FMath::Min(FMath::Max(Height, 100), DisplayMetrics.PrimaryDisplayHeight);
		}
	}

	TSharedRef<SInterchangeUsdHandlerOrderWindow> OrderWidget = SNew(SInterchangeUsdHandlerOrderWindow);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "USD Schema Handler Order"))
		.SizingRule(ESizingRule::UserSized)
		.AdjustInitialSizeAndPositionForDPIScale(false)
		.MinWidth(540.0f)  // Any smaller and it clips the title
		.ClientSize(FVector2D{static_cast<double>(Width), static_cast<double>(Height)});
	Window->SetContent(OrderWidget);
	Window->GetOnWindowClosedEvent().AddLambda(
		[OrderWidget](const TSharedRef<SWindow>& Window)
		{
			// Store our last window size on persistent storage

			// We use the geometry here because any sort of size we can extract from the window includes some added margins
			// that are not easy to account for, even with GetWindowSizeFromClientSize. The original provided ClientSize
			// seems to always match the content geometry though, so we're saving and loading the same thing
			FGeometry Geometry = Window->GetContent()->GetCachedGeometry();
			FVector2D ContentSize = Geometry.Size * Geometry.Scale;
			const int32 Width = static_cast<int32>(ContentSize.X);
			const int32 Height = static_cast<int32>(ContentSize.Y);

			GConfig->SetInt(*ConfigSectionName, *WidthConfigName, Width, GEditorPerProjectIni);
			GConfig->SetInt(*ConfigSectionName, *HeightConfigName, Height, GEditorPerProjectIni);

			const bool bRemoveFromCache = false;
			GConfig->Flush(bRemoveFromCache, GEditorPerProjectIni);
		}
	);

	OrderWidget->Settings = InSettings;
	OrderWidget->WeakWindow = Window;
	OrderWidget->Entries = ReadHandlerEntriesFromSettings(InSettings);

	// Preemptively make sure we have a progress dialog created before showing our modal. This because the progress
	// dialog itself is also modal. If it doesn't exist yet, and our options dialog causes a progress dialog
	// to be spawned (e.g. when switching the Level to export via the LevelSequenceUSDExporter), the progress dialog
	// will be pushed to the end of FSlateApplication::ActiveModalWindows (SlateApplication.cpp) and cause our options
	// dialog to pop out of its modal loop (FSlateApplication::AddModalWindow), instantly returning false to our caller
	FScopedSlowTask Progress(1, LOCTEXT("ShowingDialog", "Picking order..."));
	Progress.MakeDialogDelayed(0.25f);

	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	// Note: Being a modal window this will only return when the window closes
	const bool bSlowTaskWindow = false;
	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, bSlowTaskWindow);

	return OrderWidget->bNeedNewTranslation;
}

void SInterchangeUsdHandlerOrderWindow::Construct(const FArguments& InArgs)
{
	using namespace UE::UsdHandlerOrderWindow::Private;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MainInfoText", "All enabled handlers below are executed from top to bottom when translating each prim.\nDrag and drop to reorder them.\n\nHover this text for more info."))
				.ToolTipText(LOCTEXT("MainHoverToolTip", "A handler is a C++ class that is registered with the USD translator, and invoked when translating prims of particular schemas.\n\nHandlers that run first generally get to pick the Interchange nodes that are produced, while handlers that run later get to override data on those nodes.\n\nFor example, reorder the material handlers to pick the priority order for parsing material render contexts.\nSome material handlers also allow specifying additional render contexts that they will try parsing the material with."))
				.AutoWrapText(true)
				.Margin(10.0f)
			]

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBox)
				.Padding(FMargin(10.0f, 0.0f, 10.0f, 0.0f))
				[
					SAssignNew(ListView, SListView<TSharedPtr<FSchemaHandlerEntry>>)
					.ListItemsSource(&Entries)
					.OnGenerateRow(this, &SInterchangeUsdHandlerOrderWindow::OnGenerateListRow)
					.SelectionMode(ESelectionMode::None)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ResetToDefaultButton", "Reset To Default"))
					.OnClicked(this, &SInterchangeUsdHandlerOrderWindow::OnReset)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
						.Text(LOCTEXT("OKButton", "OK"))
						.OnClicked(this, &SInterchangeUsdHandlerOrderWindow::OnAccept)
					]
					+ SHorizontalBox::Slot()
					.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
						.Text(LOCTEXT("CancelButton", "Cancel"))
						.OnClicked(this, &SInterchangeUsdHandlerOrderWindow::OnCancel)
					]
				]
			]
		]
	];
}

TSharedRef<ITableRow> SInterchangeUsdHandlerOrderWindow::OnGenerateListRow(TSharedPtr<FSchemaHandlerEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	using namespace UE::UsdHandlerOrderWindow::Private;

	return SNew(SSchemaHandlerRow, Item, OwnerTable)
		.OnCanAcceptDrop(this, &SInterchangeUsdHandlerOrderWindow::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &SInterchangeUsdHandlerOrderWindow::OnRowAcceptDrop);
}

FReply SInterchangeUsdHandlerOrderWindow::OnAccept()
{
	using namespace UE::UsdHandlerOrderWindow::Private;

	WriteHandlerEntriesToSettings(Entries, Settings);

	// For now let's always retranslate if we hit OK, for simplicity
	bNeedNewTranslation = true;

	if (WeakWindow.IsValid())
	{
		WeakWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SInterchangeUsdHandlerOrderWindow::OnCancel()
{
	bNeedNewTranslation = false;

	if (WeakWindow.IsValid())
	{
		WeakWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SInterchangeUsdHandlerOrderWindow::OnReset()
{
	using namespace UE::UsdHandlerOrderWindow::Private;

#if USE_USD_SDK
	Entries = WrapElementsWithSharedPtr(FSchemaHandlerRegistry::RegisteredHandlerEntries);
#else
	Entries = {};
#endif	  // USE_USD_SDK

	ListView->RequestListRefresh();

	bNeedNewTranslation = true;

	return FReply::Handled();
}

FReply SInterchangeUsdHandlerOrderWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SInterchangeUsdHandlerOrderWindow::OnRowCanAcceptDrop(
	const FDragDropEvent& Event,
	EItemDropZone Zone,
	TSharedPtr<FSchemaHandlerEntry> Item
)
{
	using namespace UE::UsdHandlerOrderWindow::Private;

	if (TSharedPtr<FSchemaHandlerEntryDragDropOp> Op = Event.GetOperationAs<FSchemaHandlerEntryDragDropOp>())
	{
		if (Op->DraggedRow && Op->DraggedRow->TreeItem != Item)
		{
			// Note that we're returning BelowItem even if we're hovering OntoItem here: That makes it a bit easier
			// to use as you don't have to aim for exactly the small spaces between the items
			return Zone == EItemDropZone::AboveItem ? EItemDropZone::AboveItem : EItemDropZone::BelowItem;
		}
	}

	return {};
}

FReply SInterchangeUsdHandlerOrderWindow::OnRowAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, TSharedPtr<FSchemaHandlerEntry> Item)
{
	using namespace UE::UsdHandlerOrderWindow::Private;

	if (OnRowCanAcceptDrop(Event, Zone, Item))
	{
		if (TSharedPtr<FSchemaHandlerEntryDragDropOp> Op = Event.GetOperationAs<FSchemaHandlerEntryDragDropOp>())
		{
			const TSharedPtr<FSchemaHandlerEntry>& DraggedEntry = Op->DraggedRow->TreeItem;
			Entries.Remove(DraggedEntry);

			const int32 ItemIndex = Entries.IndexOfByKey(Item);
			if (ensure(ItemIndex != INDEX_NONE))
			{
				// Here we watch out for ending up with -1 in case we're inserting above the first entry.
				const int32 TargetIndex = FMath::Clamp(ItemIndex + (Zone == EItemDropZone::AboveItem ? 0 : 1), 0, Entries.Num());
				Entries.Insert(DraggedEntry, TargetIndex);
			}

			ListView->RebuildList();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
