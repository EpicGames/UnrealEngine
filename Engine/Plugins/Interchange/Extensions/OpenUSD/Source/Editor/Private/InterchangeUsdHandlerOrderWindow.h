// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class UInterchangeUsdTranslatorSettings;
struct FSchemaHandlerEntry;

class SInterchangeUsdHandlerOrderWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInterchangeUsdHandlerOrderWindow)
	{
	}
	SLATE_END_ARGS()

public:
	static bool ShowWindow(UInterchangeUsdTranslatorSettings* Settings);

	void Construct(const FArguments& InArgs);
	TSharedRef<ITableRow> OnGenerateListRow(TSharedPtr<FSchemaHandlerEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);

	FReply OnAccept();
	FReply OnCancel();
	FReply OnReset();
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, TSharedPtr<FSchemaHandlerEntry> Item);
	FReply OnRowAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, TSharedPtr<FSchemaHandlerEntry> Item);
	FReply OnRowDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent);
	void OnRowDragLeave(const FDragDropEvent& Event);

private:
	bool bNeedNewTranslation = false;
	UInterchangeUsdTranslatorSettings* Settings;
	TArray<TSharedPtr<FSchemaHandlerEntry>> Entries;
	TWeakPtr<SWindow> WeakWindow;
	TSharedPtr<SListView<TSharedPtr<FSchemaHandlerEntry>>> ListView;
};
