// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUsdTranslatorBlueprintLibrary.h"

#include "SchemaHandlers/SchemaHandlerRegistry.h"

TArray<FSchemaHandlerEntry> UInterchangeUsdTranslatorBlueprintLibrary::GetDefaultSchemaHandlerEntries()
{
#if USE_USD_SDK
	return UE::Interchange::USD::FSchemaHandlerRegistry::RegisteredHandlerEntries;
#else
	return {};
#endif	  // USE_USD_SDK
}
