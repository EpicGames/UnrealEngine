// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUsdTranslatorSettingsCustomization.h"

#include "InterchangeUsdHandlerOrderWindow.h"
#include "InterchangeUsdTranslator.h"
#include "UnrealUSDWrapper.h"
#include "USDMaterialUtils.h"
#include "USDProjectSettings.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "InterchangeUsdTranslatorSettingsCustomization"

FInterchangeUsdTranslatorSettingsCustomization::FInterchangeUsdTranslatorSettingsCustomization()
{
}

TSharedRef<IDetailCustomization> FInterchangeUsdTranslatorSettingsCustomization::MakeInstance()
{
	return MakeShared<FInterchangeUsdTranslatorSettingsCustomization>();
}

void FInterchangeUsdTranslatorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
	if (!SelectedObject.IsValid())
	{
		return;
	}

	CurrentOptions = Cast<UInterchangeUsdTranslatorSettings>(SelectedObject.Get());
	if (!CurrentOptions)
	{
		return;
	}

	IDetailCategoryBuilder& CatBuilder = DetailLayoutBuilder.EditCategory(TEXT("USD Translator"));

	if (TSharedPtr<IPropertyHandle> CustomHandlerProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UInterchangeUsdTranslatorSettings, CustomHandlerEntries)
		))
	{
		DetailLayoutBuilder.HideProperty(CustomHandlerProperty);

		UInterchangeUsdTranslatorSettings* Settings = CurrentOptions;

		CatBuilder.AddCustomRow(FText::FromString(TEXT("ConfigureHandlersButton")))
		.WholeRowContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(5.0f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("EditSchemaHandlerButton", "Configure schema handlers"))
				.OnClicked_Lambda([Settings, CustomHandlerProperty]()
				{
					FScopedTransaction Transaction(LOCTEXT("CustomHandlerTransaction", "Edit Custom Handlers"));
					CustomHandlerProperty->NotifyPreChange();

					bool bNeedNewTranslation = SInterchangeUsdHandlerOrderWindow::ShowWindow(Settings);

					if (bNeedNewTranslation)
					{
						// These events are important as they're the thing that eventually causes
						// SInterchangeTranslatorSettingsDialog to trigger OnTranslatorSettingsDialogClosed, and
						// ultimately cause the code in SInterchangePipelineConfigurationDialog::OnEditTranslatorSettings()
						// to trigger a new translation of the source file with the new translator (and schema handler)
						// settings when bTranslatorSettingsChanged is true
						CustomHandlerProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
						CustomHandlerProperty->NotifyFinishedChangingProperties();
					}

					return FReply::Handled();
				})
			]
		];
	}

	if (TSharedPtr<IPropertyHandle> MaterialPurposeProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UInterchangeUsdTranslatorSettings, MaterialPurpose)
		))
	{
		DetailLayoutBuilder.HideProperty(MaterialPurposeProperty);

		// clang-format off
		CatBuilder.AddCustomRow(FText::FromString(TEXT("MaterialPurposeCustomization")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Material Purpose")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(MaterialPurposeProperty->GetToolTipText())
		]
		.ValueContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&MaterialPurposeComboBoxItems)
				.OnComboBoxOpening_Lambda([this]()
				{
					MaterialPurposeComboBoxItems = {
						MakeShared<FString>(UnrealIdentifiers::MaterialAllPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialPreviewPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialFullPurpose)
					};

					// Add additional purposes from project settings
					if (const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>())
					{
						MaterialPurposeComboBoxItems.Reserve(MaterialPurposeComboBoxItems.Num() + ProjectSettings->AdditionalMaterialPurposes.Num());

						TSet<FString> ExistingEntries = {
							UnrealIdentifiers::MaterialAllPurpose,
							UnrealIdentifiers::MaterialPreviewPurpose,
							UnrealIdentifiers::MaterialFullPurpose
						};

						for (const FName& AdditionalPurpose : ProjectSettings->AdditionalMaterialPurposes)
						{
							FString AdditionalPurposeStr = AdditionalPurpose.ToString();

							if (!ExistingEntries.Contains(AdditionalPurposeStr))
							{
								ExistingEntries.Add(AdditionalPurposeStr);
								MaterialPurposeComboBoxItems.AddUnique(MakeShared<FString>(AdditionalPurposeStr));
							}
						}
					}
				})
				.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
				{
					TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
					if (Option)
					{
						Widget = SNew(STextBlock)
							.Text(FText::FromString((*Option) == UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: *Option
							))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
					}

					return Widget.ToSharedRef();
				})
				.OnSelectionChanged_Lambda([this, MaterialPurposeProperty](TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
				{
					if (CurrentOptions && ChosenOption)
					{
						FScopedTransaction Transaction(LOCTEXT("MaterialPurposeTransaction", "Edit Material Purpose"));
						MaterialPurposeProperty->NotifyPreChange();
						{
							CurrentOptions->MaterialPurpose = **ChosenOption;
						}
						MaterialPurposeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
						MaterialPurposeProperty->NotifyFinishedChangingProperties();
					}

				})
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]() -> FText
					{
						if (CurrentOptions)
						{
							return FText::FromString(CurrentOptions->MaterialPurpose == *UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: CurrentOptions->MaterialPurpose.ToString()
							);
						}

						return FText::GetEmpty();
					})
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.OnTextCommitted_Lambda([this, MaterialPurposeProperty](const FText& NewText, ETextCommit::Type CommitType)
					{
						if (CommitType != ETextCommit::OnEnter)
						{
							return;
						}

						FString NewPurposeString = NewText.ToString();
						FName NewPurpose = *NewPurposeString;

						bool bIsNew = true;
						for (const TSharedPtr<FString>& Purpose : MaterialPurposeComboBoxItems)
						{
							if (Purpose && *Purpose == NewPurposeString)
							{
								bIsNew = false;
								break;
							}
						}

						if (bIsNew)
						{
							if (UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>())
							{
								ProjectSettings->AdditionalMaterialPurposes.AddUnique(NewPurpose);
								ProjectSettings->SaveConfig();
							}
						}

						if (CurrentOptions)
						{
							FScopedTransaction Transaction(LOCTEXT("MaterialPurposeTypeTransaction", "Add and Set Material Purpose"));
							MaterialPurposeProperty->NotifyPreChange();
							{
								CurrentOptions->MaterialPurpose = NewPurpose;
							}
							MaterialPurposeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
							MaterialPurposeProperty->NotifyFinishedChangingProperties();
						}
					})
				]
			]
		];
		// clang-format on
	}

	// Add/remove properties so that they retain their usual order
	if (TSharedPtr<IPropertyHandle> OverrideStageOptionsProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UInterchangeUsdTranslatorSettings, bOverrideStageOptions)
		))
	{
		CatBuilder.AddProperty(OverrideStageOptionsProperty);
	}
	if (TSharedPtr<IPropertyHandle> StageOptionsProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UInterchangeUsdTranslatorSettings, StageOptions)
		))
	{
		CatBuilder.AddProperty(StageOptionsProperty);
	}
}

void FInterchangeUsdTranslatorSettingsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CustomizeDetails(*DetailBuilder);
}

#undef LOCTEXT_NAMESPACE
