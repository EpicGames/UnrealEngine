// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaHandlers/SchemaHandlerEntry.h"

#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "InterchangeUsdTranslatorBlueprintLibrary.generated.h"

#define UE_API INTERCHANGEOPENUSDEDITOR_API

UCLASS(MinimalAPI, meta = (ScriptName = "InterchangeUsdTranslatorLibrary"))
class UInterchangeUsdTranslatorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Returns the default list of schema handlers entries registered via C++, i.e. FSchemaHandlerRegistry::RegisteredHandlerEntries.
	 *
	 * The main use case here is to retrieve this list, modify it as needed, and to set it as the CustomHandlerEntries property on
	 * the UInterchangeUsdTranslatorSettings class default object, which will be used by any Interchange USD translations after
	 * that point
	 */
	UFUNCTION(BlueprintCallable, Category = "USD Translator")
	static UE_API TArray<FSchemaHandlerEntry> GetDefaultSchemaHandlerEntries();
};

#undef UE_API
