// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeOpenUSDImport : ModuleRules
	{
		public InterchangeOpenUSDImport(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InterchangeCommon",
					"InterchangeCommonParser",	// For FAnimationPayloadData
					"InterchangeCore", 			// For UInterchangeBaseNode, base translator
					"InterchangeImport", 		// For the translator settings, pipeline interface, etc.
					"UnrealUSDWrapper",
					"USDUtilities"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DynamicMesh",				// For FMeshMapMaker
					"GeometryCore",				// For FDynamicMesh3
					"HairStrandsCore",
					"InterchangeEngine", 		// For UInterchangeManager::GetInterchangeManager
					"InterchangeFactoryNodes",
					"InterchangeNodes", 		// For UInterchangeShaderGraphNode
					"InterchangePipelines",
					"MeshConversion",			// For FMeshDescriptionToDynamicMesh
					"MeshDescription",
					"StaticMeshDescription",	// FStaticMeshOperations
					"TextureUtilitiesCommon",	// For UDIM functions
					"USDClasses"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"InterchangeOpenVDBImport",
						"MaterialX",
						"ModelingComponents",	// For FTexture2DBuilder
					}
				);
			}
		}
	}
}