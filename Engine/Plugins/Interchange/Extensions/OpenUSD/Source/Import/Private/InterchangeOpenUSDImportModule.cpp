// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeOpenUSDImportModule.h"

#include "InterchangeUsdTranslator.h"
#include "SchemaHandlers/CameraSchemaHandler.h"
#include "SchemaHandlers/CollapsingSchemaHandler.h"
#include "SchemaHandlers/GeometryCacheSchemaHandler.h"
#include "SchemaHandlers/GprimSchemaHandler.h"
#include "SchemaHandlers/GroomBindingSchemaHandler.h"
#include "SchemaHandlers/GroomSchemaHandler.h"
#include "SchemaHandlers/ImageableSchemaHandler.h"
#include "SchemaHandlers/LightSchemaHandler.h"
#include "SchemaHandlers/MaterialXSchemaHandler.h"
#include "SchemaHandlers/NaniteAssemblySchemaHandler.h"
#include "SchemaHandlers/PointInstancerSchemaHandler.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/SchemaHandlerRegistry.h"
#include "SchemaHandlers/SkeletonSchemaHandler.h"
#include "SchemaHandlers/SpatialAudioSchemaHandler.h"
#include "SchemaHandlers/UniversalMaterialSchemaHandler.h"
#include "SchemaHandlers/UnrealMaterialSchemaHandler.h"
#include "SchemaHandlers/VolumeSchemaHandler.h"
#include "SchemaHandlers/XformableSchemaHandler.h"

#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogInterchangeUSDModule, Log, All);

void FInterchangeOpenUSDImportModule::StartupModule()
{
#if USE_USD_SDK
	using namespace UE::Interchange;
	using namespace UE::Interchange::USD;

	// This is where the default schema handlers are registered.
	//
	// Note that the order here is important, and the handlers will always be executed in this order during
	// translation and payload retrieval.
	//
	// In general, handlers that run first get to pick the node types that are created for a prim,
	// while handlers that run later get to override what is put on those nodes.
	RegisteredHandlers.Append({
		// Special / unusual handlers
		FSchemaHandlerRegistry::Register<FCollapsingSchemaHandler>(), // Going before all the other handlers lets it block regular translation if needed
		FSchemaHandlerRegistry::Register<FNaniteAssemblySchemaHandler>(),
		FSchemaHandlerRegistry::Register<FSkeletonSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FGroomSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FGroomBindingSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FPointInstancerSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FCameraSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FLightSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FSpatialAudioSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FVolumeSchemaHandler>(),

		// Common mesh / xform handlers
		FSchemaHandlerRegistry::Register<FImageableSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FXformableSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FGeometryCacheSchemaHandler>(), // This needs to be before Gprim handler to create the geometry cache node first
		FSchemaHandlerRegistry::Register<FGprimSchemaHandler>(),

		// Material handlers
		FSchemaHandlerRegistry::Register<FMaterialXSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FUnrealMaterialSchemaHandler>(),
		FSchemaHandlerRegistry::Register<FUniversalMaterialSchemaHandler>(), // This should be after other material render context handlers as it will only create material nodes if it doesn't find any other
	});

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	InterchangeManager.RegisterTranslator(UInterchangeUSDTranslator::StaticClass());

	// Don't go through FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings as we'll need a member of the actual
	// FInterchangeContent/SceneImportSettings struct anyway, and this is likely safer than casting the struct pointer
	UInterchangeProjectSettings* ProjectSettings = GetMutableDefault<UInterchangeProjectSettings>();
	if (!ProjectSettings)
	{
		return;
	}

	FInterchangeContentImportSettings& AssetImportSettings = ProjectSettings->ContentImportSettings;
	FInterchangeSceneImportSettings& SceneImportSettings = ProjectSettings->SceneImportSettings;

	TSoftClassPtr<UInterchangeTranslatorBase> TranslatorClassPath = TSoftClassPtr<UInterchangeTranslatorBase>(UInterchangeUSDTranslator::StaticClass());

	FInterchangePerTranslatorDialogOverride ImportDialogOverride;
	ImportDialogOverride.Translator = TranslatorClassPath;
	ImportDialogOverride.bShowImportDialog = true;
	ImportDialogOverride.bShowReimportDialog = true;

	// Asset import pipelines
	{
		const FName DefaultPipelineStack = AssetImportSettings.DefaultPipelineStack;
		if (AssetImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
		{
			FInterchangeTranslatorPipelines TranslatorPipelines;
			TranslatorPipelines.Translator = TranslatorClassPath;
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeOpenUSD/Pipelines/DefaultUSDAssetsPipeline.DefaultUSDAssetsPipeline")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeOpenUSD/Pipelines/DefaultUSDPipelineAssetImport.DefaultUSDPipelineAssetImport")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/Interchange/Pipelines/DefaultAudioPipeline.DefaultAudioPipeline")));

			FInterchangePipelineStack& PipelineStack = AssetImportSettings.PipelineStacks[DefaultPipelineStack];
			PipelineStack.PerTranslatorPipelines.Add(TranslatorPipelines);

			UE_LOGF(LogInterchangeUSDModule, Log, "InterchangeUSD's pipelines added to default Asset pipeline stack, %ls", *DefaultPipelineStack.ToString());
		}
		else
		{
			UE_LOGF(LogInterchangeUSDModule, Warning, "Cannot find Interchange default Asset's stack to add pipelines. InterchangeUSD asset import may not work properly.");
		}
	}

	// Scene import pipelines
	{
		const FName DefaultPipelineStack = SceneImportSettings.DefaultPipelineStack;
		if (SceneImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
		{
			FInterchangeTranslatorPipelines TranslatorPipelines;
			TranslatorPipelines.Translator = TranslatorClassPath;
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeOpenUSD/Pipelines/DefaultUSDSceneAssetsPipeline.DefaultUSDSceneAssetsPipeline")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeOpenUSD/Pipelines/DefaultUSDSceneLevelPipeline.DefaultUSDSceneLevelPipeline")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeOpenUSD/Pipelines/DefaultUSDPipeline.DefaultUSDPipeline")));
			TranslatorPipelines.Pipelines.Add(FSoftObjectPath(TEXT("/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline")));

			FInterchangePipelineStack& PipelineStack = SceneImportSettings.PipelineStacks[DefaultPipelineStack];
			PipelineStack.PerTranslatorPipelines.Add(TranslatorPipelines);

			UE_LOGF(LogInterchangeUSDModule, Log, "InterchangeUSD's pipelines added to default Scene pipeline stack, %ls", *DefaultPipelineStack.ToString());
		}
		else
		{
			UE_LOGF(LogInterchangeUSDModule, Warning, "Cannot find Interchange default Scene's stack to add pipelines. InterchangeUSD level import may not work properly.");
		}
	}

	// Asset import dialog overrides (we want to show the import and reimport dialog for all USD imports, of all asset types)
	{
		// TODO: Add a Count value to the enum and then use ENUM_RANGE_BY_COUNT?
		static const TArray<EInterchangeTranslatorAssetType> AssetTypes = {
			EInterchangeTranslatorAssetType::Textures,
			EInterchangeTranslatorAssetType::Materials,
			EInterchangeTranslatorAssetType::Meshes,
			EInterchangeTranslatorAssetType::Animations,
			EInterchangeTranslatorAssetType::Grooms,
		};

		for (EInterchangeTranslatorAssetType AssetType : AssetTypes)
		{
			FInterchangeDialogOverride& DialogOverrides = AssetImportSettings.ShowImportDialogOverride.FindOrAdd(AssetType);

			// Don't set the dialog overrides for the USD translator if the user already has done that (with potentially different values)
			bool bHasUSDOverrides = false;
			for (const FInterchangePerTranslatorDialogOverride& Override : DialogOverrides.PerTranslatorImportDialogOverride)
			{
				if (Override.Translator == TranslatorClassPath)
				{
					bHasUSDOverrides = true;
					break;
				}
			}

			if (!bHasUSDOverrides)
			{
				DialogOverrides.PerTranslatorImportDialogOverride.Add(ImportDialogOverride);
			}
		}
	}

	// Scene import dialog overrides
	{
		// Don't set the dialog overrides for the USD translator if the user already has done that (with potentially different values)
		bool bHasUSDOverrides = false;
		for (const FInterchangePerTranslatorDialogOverride& Override : SceneImportSettings.PerTranslatorDialogOverride)
		{
			if (Override.Translator == TranslatorClassPath)
			{
				bHasUSDOverrides = true;
				break;
			}
		}

		if (!bHasUSDOverrides)
		{
			SceneImportSettings.PerTranslatorDialogOverride.Add(ImportDialogOverride);
		}
	}

#endif	  // USE_USD_SDK
}
void FInterchangeOpenUSDImportModule::ShutdownModule()
{
#if USE_USD_SDK
	using namespace UE::Interchange;
	using namespace UE::Interchange::USD;

	for (const FString& Name : RegisteredHandlers)
	{
		FSchemaHandlerRegistry::Unregister(Name);
	}
#endif	  // USE_USD_SDK
}

int32 FInterchangeOpenUSDImportModule::GetNumDefaultHandlers() const
{
#if USE_USD_SDK
	return RegisteredHandlers.Num();
#else
	return 0;
#endif	  // USE_USD_SDK
}

IMPLEMENT_MODULE(FInterchangeOpenUSDImportModule, InterchangeOpenUSDImport)