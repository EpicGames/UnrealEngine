// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "InterchangeUSDInfoCache.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDIntegrationUtils.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Async/ParallelFor.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "UsdInfoCache"
#define DEFAULT_GEOMETRY_CACHE_MAX_DEPTH 15
#define DEFAULT_NUM_PER_PRIM_LOCKS 32

namespace UE::UsdInfoCache::Private
{
	// We fetch these from the cvar once before every Build()
	static bool UseGeometryCacheUSD = true;
	static int32 GeometryCacheMaxDepth = DEFAULT_GEOMETRY_CACHE_MAX_DEPTH;
	static int32 NumPerPrimLocks = DEFAULT_NUM_PER_PRIM_LOCKS;

	// Flags to hint at the state of a prim for the purpose of geometry cache
	enum class EGeometryCachePrimState : uint8
	{
		None = 0x00,
		Uncollapsible = 0x01,		   // prim cannot be collapsed as part of a geometry cache
		Mesh = 0x02,				   // prim is a mesh, animated or not
		Xform = 0x04,				   // prim is a xform, animated or not
		Collapsible = Mesh | Xform,	   // only meshes and xforms can be collapsed into a geometry cache
		ValidRoot = 0x08			   // prim can collapse itself and its children into a geometry cache
	};
	ENUM_CLASS_FLAGS(EGeometryCachePrimState)

	struct FUsdPrimInfo
	{
		UE::FSdfPath PrimPath;

		uint64 ParentInfoIndex = static_cast<uint64>(INDEX_NONE);
		TArray<uint64> ChildIndices;

		int32 PrimLockIndex = INDEX_NONE;

		// If this is true, it means this prim can and wants to collapse its entire subtree.
		// If false, it either doesn't collapse its subtree, or we haven't visited it yet (same result)
		bool bCollapsesChildren = false;

		// Whether this prim can be collapsed or not, according to its schema handlers
		// - Optional is not set: Prim wasn't visited yet, we don't know
		// - Optional has value: Whether the prim can be collapsed or not
		TOptional<bool> bXformSubtreeCanBeCollapsed;

		int32 GeometryCacheDepth = -1;
		EGeometryCachePrimState GeometryCacheState = EGeometryCachePrimState::None;
	};

	struct FInterchangeUsdInfoCacheImpl
	{
		FInterchangeUsdInfoCacheImpl()
			: AllowedExtensionsForGeometryCacheSource(UnrealUSDWrapper::GetNativeFileFormats())
		{
			AllowedExtensionsForGeometryCacheSource.Add(TEXT("abc"));
		}

		~FInterchangeUsdInfoCacheImpl()
		{
			delete[] PrimLocks;
		}

		bool bIsBuilding = false;

		// Information we must have about all prims on the stage
		TArray<UE::UsdInfoCache::Private::FUsdPrimInfo> PrimInfoArray;
		TMap<UE::FSdfPath, uint64> InfoMap;
		mutable FRWLock InfoMapLock;

		// Geometry cache can come from a reference or payload of these file types
		TArray<FString> AllowedExtensionsForGeometryCacheSource;

	private:
		// Individual locks distributed across the FUsdPrimInfo
		FRWLock* PrimLocks = nullptr;

	public:
		// WARNING: Assumes that the info map is locked for reading
		UE::UsdInfoCache::Private::FUsdPrimInfo* GetPrimInfo(const UE::FSdfPath& PrimPath) const
		{
			if (const uint64* Index = InfoMap.Find(PrimPath))
			{
				return const_cast<UE::UsdInfoCache::Private::FUsdPrimInfo*>(&PrimInfoArray[*Index]);
			}

			return nullptr;
		}

		void ReallocatePrimLocks()
		{
			delete[] PrimLocks;
			PrimLocks = new FRWLock[UE::UsdInfoCache::Private::NumPerPrimLocks]();
		};

		[[nodiscard]] FReadScopeLock LockForReading(const UE::UsdInfoCache::Private::FUsdPrimInfo& Info) const
		{
			return FReadScopeLock{PrimLocks[Info.PrimLockIndex]};
		}

		[[nodiscard]] FWriteScopeLock LockForWriting(const UE::UsdInfoCache::Private::FUsdPrimInfo& Info) const
		{
			return FWriteScopeLock{PrimLocks[Info.PrimLockIndex]};
		}

		UE::UsdInfoCache::Private::FUsdPrimInfo& CreateNewInfo(const UE::FSdfPath& PrimPath, uint64& OutNewIndex)
		{
			using namespace UE::UsdInfoCache::Private;

			OutNewIndex = PrimInfoArray.Num();

			FUsdPrimInfo& Result = PrimInfoArray.Emplace_GetRef();
			Result.PrimPath = PrimPath;
			return Result;
		}

		bool IsPotentialGeometryCacheRootInner(UE::UsdInfoCache::Private::FUsdPrimInfo& Info, const pxr::UsdPrim& Prim)
		{
			using namespace UE::UsdInfoCache::Private;

			FWriteScopeLock PrimLock = LockForWriting(Info);

			// When importing we fill all those in during the info cache initial build. If this is None still, it means
			// we're in the default geometry cache workflow for opening the stage, where geometry caches are generated
			// directly for single animated Mesh prims (so no collapsing of whole subtrees into geometry caches). We can
			// then find out if our prim is animated on-demand
			if (Info.GeometryCacheState == EGeometryCachePrimState::None)
			{
				Info.GeometryCacheState = UsdUtils::IsAnimatedMesh(Prim) ? EGeometryCachePrimState::ValidRoot
																		 : EGeometryCachePrimState::Uncollapsible;
			}
			return Info.GeometryCacheState == EGeometryCachePrimState::ValidRoot;
		}
	};

	void PreallocateForSubtree(const UE::FSdfPath& SubtreeRootPath, const UE::FUsdStage& Stage, FInterchangeUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::PreallocateForSubtree);

		if (!Stage)
		{
			return;
		}

		UE::FUsdPrim SubtreeRootPrim = Stage.GetPrimAtPath(SubtreeRootPath);
		if (!SubtreeRootPrim)
		{
			// It's possible to be called with paths to prims that don't exist on the stage, for example when handling the
			// rebuild about removing a prim spec, where USD sends a resync notice for the path to the prim that was just removed.
			// We still want the info cache build to do all the rest in those cases though (cleanup old entries, invalidate ancestors,
			// etc.), so we handle ignoring this case only in here
			return;
		}

		TFunction<uint64(const UE::FUsdPrim&, uint64)> ConstructInfoForPrim;
		ConstructInfoForPrim = [&Impl, &ConstructInfoForPrim](const UE::FUsdPrim& Prim, uint64 ParentIndex) -> uint64
		{
			// Note: We're not locking the infos here at all as our access pattern will never touch the same info more than once anyway,
			// and this function is single threaded and never calls in to any other thread-unsafe functions
			uint64 NewIndex = INDEX_NONE;
			FUsdPrimInfo& NewInfo = Impl.CreateNewInfo(UE::FSdfPath{Prim.GetPrimPath()}, NewIndex);
			NewInfo.PrimLockIndex = static_cast<int32>(NewIndex % UE::UsdInfoCache::Private::NumPerPrimLocks);
			NewInfo.ParentInfoIndex = ParentIndex;

			Impl.InfoMap.Add(NewInfo.PrimPath, NewIndex);

			// Note: I've tried a ParallelFor here, and it was slower than the single threaded version due to write lock
			// contention on the InfoMap itself
			TArray<uint64> ChildIndices;
			const bool bTraverseInstanceProxies = true;
			for (const UE::FUsdPrim& Child : Prim.GetFilteredChildren(bTraverseInstanceProxies))
			{
				const uint64 ChildIndex = ConstructInfoForPrim(Child, NewIndex);
				ChildIndices.Add(ChildIndex);
			}
			// Have to find our NewInfo again as the recursive calls likely invalidated our NewInfo reference
			Impl.PrimInfoArray[NewIndex].ChildIndices = MoveTemp(ChildIndices);

			return NewIndex;
		};

		FWriteScopeLock ScopeLock(Impl.InfoMapLock);

		// Find parent
		UE::FSdfPath ParentPrimPath = SubtreeRootPath.GetParentPath();
		uint64* ParentIndexPtr = Impl.InfoMap.Find(ParentPrimPath);
		uint64 ParentIndex = ParentIndexPtr ? *ParentIndexPtr : INDEX_NONE;

		// Create new subtree
		uint64 SubtreeRootIndex = ConstructInfoForPrim(SubtreeRootPrim, ParentIndex);

		// Connect the new subtree to its target parent
		if (ParentIndex != INDEX_NONE)
		{
			Impl.PrimInfoArray[ParentIndex].ChildIndices.Add(SubtreeRootIndex);
		}
	}

	bool CanBeCollapsedFallback(const UE::FUsdPrim& UsdPrim, UInterchangeUsdContext& Context)
	{
		// Unlike for CollapsesChildrenFallaback, we're not super strict here so that we can e.g. collapse subtrees with
		// meshes that have UsdGeomSubsets in them
		if (!UsdPrim)
		{
			return false;
		}

		if (UsdUtils::IsAnimated(UsdPrim) ||
			UsdUtils::PrimHasSchema(UsdPrim, UnrealIdentifiers::LiveLinkAPI))
		{
			return false;
		}

		UInterchangeUsdTranslatorSettings* Settings = Context.GetTranslatorSettings();
		if (!Settings)
		{
			return false;
		}

		// If we're not using prim kinds to collapse, we still want things to *be collapsible* by default (so that they can be controlled with the
		// collapsing attributes)
		const bool bCanCollapseByDefault = true;
		return UsdUtils::PrimCollapses(
			UsdPrim,
			static_cast<EUsdDefaultKind>(Settings->KindsToCollapse),
			Settings->bUsePrimKindsForCollapsing,
			Settings->bUseSchemaForCollapsing,
			bCanCollapseByDefault
		);
	}

	bool RecursiveQueryCanBeCollapsed(const UE::FUsdPrim& UsdPrim, UInterchangeUsdContext& Context, FInterchangeUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::RecursiveQueryCanBeCollapsed);

		UE::FSdfPath UsdPrimPath = UE::FSdfPath{UsdPrim.GetPrimPath()};

		FReadScopeLock ScopeLock{Impl.InfoMapLock};

		// If we already have a value for our prim then we can just return it right now. We only fill these bCanBeCollapsed values
		// through here, so if we know e.g. that UsdPrim can be collapsed, we know its entire subtree can too.
		FUsdPrimInfo* MainPrimInfo = Impl.GetPrimInfo(UsdPrimPath);
		if (MainPrimInfo)
		{
			FReadScopeLock PrimLock = Impl.LockForReading(*MainPrimInfo);
			if (MainPrimInfo->bXformSubtreeCanBeCollapsed.IsSet())
			{
				return MainPrimInfo->bXformSubtreeCanBeCollapsed.GetValue();
			}
		}

		UInterchangeUSDTranslator* Translator = Context.GetTranslator();
		if (!ensure(Translator))
		{
			return false;
		}

		TOptional<bool> bCanBeCollapsed;
		for (const TSharedRef<UE::Interchange::USD::FSchemaHandler>& Handler : Translator->GetCurrentSchemaHandlers())
		{
			// We check this here, otherwise we have to rely on the schema handler's CollapsesChildren() implementation to itself
			// first check whether the handler should be queried in this case... It would for example be very misleading if we just
			// implemented FCameraSchemaHandler::CanBeCollapsed() to always return false: It would look like we're saying that cameras
			// can't be collapsed, but we'd be secretly preventing all prims from collapsing...
			if (!Handler->CanHandlePrim(UsdPrim, Context))
			{
				continue;
			}

			TOptional<bool> HandlerCanBeCollapsed = Handler->CanBeCollapsed(UsdPrim, Context);
			if (HandlerCanBeCollapsed.IsSet())
			{
				bCanBeCollapsed = HandlerCanBeCollapsed.GetValue();
				break;
			}
		}

		if (!bCanBeCollapsed.IsSet())
		{
			bCanBeCollapsed = CanBeCollapsedFallback(UsdPrim, Context);
		}

		// If we can be collapsed ourselves we're not still done, because this is about the subtree. If any of our
		// children can't be collapsed, we actually can't either
		ensure(bCanBeCollapsed.IsSet());
		if (bCanBeCollapsed.GetValue())
		{
			TArray<UE::FUsdPrim> Children;

			const bool bTraverseInstanceProxies = true;
			for (UE::FUsdPrim Child : UsdPrim.GetFilteredChildren(bTraverseInstanceProxies))
			{
				// We don't care about non-GeomImagable prims
				// (materials, etc., stuff we don't have schema handlers for will be skipped and default to bCanBeCollapsed=true)
				if (Child.IsA(TEXT("Imageable")))
				{
					Children.Emplace(Child);
				}
			}

			TArray<bool> ChildrenCanBeCollapsed;
			ChildrenCanBeCollapsed.SetNumZeroed(Children.Num());

			const int32 MinBatchSize = 1;
			ParallelFor(
				TEXT("RecursiveQueryCanBeCollapsed"),
				Children.Num(),
				MinBatchSize,
				[&](int32 Index)
				{
					ChildrenCanBeCollapsed[Index] = RecursiveQueryCanBeCollapsed(Children[Index], Context, Impl);
				}
			);

			for (bool bChildCanBeCollapsed : ChildrenCanBeCollapsed)
			{
				if (!bChildCanBeCollapsed)
				{
					bCanBeCollapsed = false;
					break;
				}
			}
		}

		// Record what we found about our main prim
		if (MainPrimInfo)
		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(*MainPrimInfo);
			MainPrimInfo->bXformSubtreeCanBeCollapsed = bCanBeCollapsed.GetValue();
		}

		// Before we return though, what we can do here is that if we know that we can't be collapsed ourselves,
		// then none of our ancestors can either! So let's quickly paint upwards to make future queries faster
		if (!bCanBeCollapsed.GetValue() && MainPrimInfo)
		{
			uint64 IterIndex = INDEX_NONE;
			{
				IterIndex = MainPrimInfo->ParentInfoIndex;
			}

			while (IterIndex != INDEX_NONE)
			{
				FUsdPrimInfo& AncestorInfo = Impl.PrimInfoArray[IterIndex];
				FWriteScopeLock PrimLock = Impl.LockForWriting(*MainPrimInfo);

				// We found something that was already filled out: Let's stop traversing here.
				//
				// We should really never find an ancestor that has been marked as collapsible in this case, but
				// if we do let's try to recover a little bit and continue painting upwards that we can't collapse
				if (AncestorInfo.bXformSubtreeCanBeCollapsed.IsSet() && ensure(AncestorInfo.bXformSubtreeCanBeCollapsed.GetValue() == false))
				{
					break;
				}
				else
				{
					AncestorInfo.bXformSubtreeCanBeCollapsed = false;
				}

				IterIndex = AncestorInfo.ParentInfoIndex;
			}
		}

		return bCanBeCollapsed.GetValue();
	}

	bool CanPrimSubtreeBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& Context, FInterchangeUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::CanSubtreeBeCollapsed);

		FReadScopeLock ScopeLock(Impl.InfoMapLock);

		if (const FUsdPrimInfo* FoundInfo = Impl.GetPrimInfo(Prim.GetPrimPath()))
		{
			FReadScopeLock PrimLock = Impl.LockForReading(*FoundInfo);
			if (FoundInfo->bXformSubtreeCanBeCollapsed.IsSet())
			{
				return FoundInfo->bXformSubtreeCanBeCollapsed.GetValue();
			}
		}

		return RecursiveQueryCanBeCollapsed(Prim, Context, Impl);
	}

	bool CollapsesChildrenFallback(const UE::FUsdPrim& Prim, UInterchangeUsdContext& Context, FInterchangeUsdInfoCacheImpl& Impl)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::CollapsesChildren);

		// Since this is just a fallback, let's by default prevent non-imageable stuff from collapsing, as that doesn't even make 
		// much sense anyway (e.g. Shader prims, UsdGeomSubsets, etc.). If the user has some odd case where they want this to happen
		// they can easily override this by authoring any custom schema handler for their prim type
		if (!Prim || Prim.IsPseudoRoot() || !Prim.IsA(TEXT("Imageable")))
		{
			return false;
		}

		UInterchangeUsdTranslatorSettings* Settings = Context.GetTranslatorSettings();
		if (!Settings)
		{
			return false;
		}

		// If we're not using prim kinds to collapse, we don't want anything to *try collapsing its children* by default
		const bool bCanCollapseByDefault = false;

		// To collapse our children we must be able to collapse in the first place
		bool bCollapses = UsdUtils::PrimCollapses(
			Prim,
			static_cast<EUsdDefaultKind>(Settings->KindsToCollapse),
			Settings->bUsePrimKindsForCollapsing,
			Settings->bUseSchemaForCollapsing,
			bCanCollapseByDefault
		);

		if (bCollapses)
		{
			// We know we can collapse Prim itself. Now we need to check if its subtree can be collapsed
			return CanPrimSubtreeBeCollapsed(Prim, Context, Impl);
		}

		return bCollapses;
	}

	void RecursiveQueryCollapsesChildren(uint64 PrimIndex, UInterchangeUsdContext& Context, FInterchangeUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::RecursiveQueryCollapsesChildren);

		FReadScopeLock ScopeLock{Impl.InfoMapLock};
		FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];
		{
			FReadScopeLock PrimLock = Impl.LockForReading(Info);
			if (Info.bCollapsesChildren)
			{
				return;
			}
		}

		UE::FUsdPrim UsdPrim = Context.GetUsdStage().GetPrimAtPath(Info.PrimPath);
		if (!UsdPrim)
		{
			return;
		}

		// TODO: This is analogous to legacy USD, but should this criteria be stronger than all the schema handler opinions here too?
		TOptional<bool> bCollapsesChildren;
		if (Impl.IsPotentialGeometryCacheRootInner(Info, UsdPrim))
		{
			bCollapsesChildren = true;
		}
		else
		{
			if (UInterchangeUSDTranslator* Translator = Context.GetTranslator())
			{
				for (const TSharedRef<UE::Interchange::USD::FSchemaHandler>& Handler : Translator->GetCurrentSchemaHandlers())
				{
					// We check this here, otherwise we have to rely on the schema handler's CollapsesChildren() implementation to itself
					// first check whether the handler should be queried in this case... It would for example be very misleading if we just
					// implemented FCameraSchemaHandler::CanBeCollapsed() to always return false: It would look like we're saying that cameras
					// can't be collapsed, but we'd be secretly preventing all prims from collapsing...
					if (!Handler->CanHandlePrim(UsdPrim, Context))
					{
						continue;
					}

					TOptional<bool> HandlerCollapsesChildren = Handler->CollapsesChildren(UsdPrim, Context);
					if (HandlerCollapsesChildren.IsSet())
					{
						bCollapsesChildren = HandlerCollapsesChildren.GetValue();
						break;
					}
				}
			}
		}

		// No strong opinions about whether this prim collapses children or not --> Fallback to default behavior
		if (!bCollapsesChildren.IsSet())
		{
			bCollapsesChildren = CollapsesChildrenFallback(UsdPrim, Context, Impl);
		}

		ensure(bCollapsesChildren.IsSet());
		if (bCollapsesChildren.GetValue())
		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(Info);
			Info.bCollapsesChildren = true;
		}
		// We only need to visit our children if we don't collapse. We'll leave the fields unset on the InfoMap,
		// and whenever we query info about a particular prim will fill that in on-demand by just traveling
		// upwards until we run into our collapse root
		else
		{
			const int32 MinBatchSize = 1;
			ParallelFor(
				TEXT("RecursiveQueryCollapsesChildren"),
				Info.ChildIndices.Num(),
				MinBatchSize,
				[&](int32 Index)
				{
					RecursiveQueryCollapsesChildren(Info.ChildIndices[Index], Context, Impl);
				}
			);
		}
	}

	void FindValidGeometryCacheRoot(
		const UE::FUsdPrim& UsdPrim,
		UInterchangeUsdContext& Context,
		FInterchangeUsdInfoCacheImpl& Impl,
		UE::UsdInfoCache::Private::EGeometryCachePrimState& OutState
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::FindValidGeometryCacheRoot);

		using namespace UE::UsdInfoCache::Private;

		UE::FSdfPath UsdPrimPath = UsdPrim.GetPrimPath();
		UE::FSdfPath PrimPath{UsdPrimPath};
		{
			FReadScopeLock ScopeLock(Impl.InfoMapLock);
			if (UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UE::FSdfPath(UsdPrim.GetPrimPath())))
			{
				FWriteScopeLock PrimLock = Impl.LockForWriting(*Info);

				// A prim is considered a valid root if its subtree has no uncollapsible branch and a valid depth.
				// A valid depth is positive, meaning it has an animated mesh, and doesn't exceed the limit.
				bool bIsValidDepth = Info->GeometryCacheDepth > -1 && Info->GeometryCacheDepth <= UE::UsdInfoCache::Private::GeometryCacheMaxDepth;
				if (!EnumHasAnyFlags(Info->GeometryCacheState, EGeometryCachePrimState::Uncollapsible) && bIsValidDepth)
				{
					OutState = EGeometryCachePrimState::ValidRoot;
					Info->GeometryCacheState = EGeometryCachePrimState::ValidRoot;
					return;
				}
				// The prim is not a valid root so it's flagged as uncollapsible since the root will be among its children
				// and the eventual geometry cache cannot be collapsed.
				else
				{
					OutState = EGeometryCachePrimState::Uncollapsible;
					Info->GeometryCacheState = EGeometryCachePrimState::Uncollapsible;
				}
			}
		}

		// Continue the search for a valid root among the children
		const bool bTraverseInstanceProxies = true;
		for (const UE::FUsdPrim& Child : UsdPrim.GetFilteredChildren(bTraverseInstanceProxies))
		{
			bool bIsCollapsible = false;
			{
				FReadScopeLock ScopeLock(Impl.InfoMapLock);
				if (const UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UE::FSdfPath(Child.GetPrimPath())))
				{
					FReadScopeLock Lock = Impl.LockForReading(*Info);

					bIsCollapsible = EnumHasAnyFlags(Info->GeometryCacheState, EGeometryCachePrimState::Collapsible);
				}
			}

			// A subtree is considered only if it has anything collapsible in the first place
			if (bIsCollapsible)
			{
				FindValidGeometryCacheRoot(Child, Context, Impl, OutState);
			}
		}

		OutState = EGeometryCachePrimState::Uncollapsible;
	}

	void RecursiveCheckForGeometryCache(
		uint64 PrimIndex,
		UInterchangeUsdContext& Context,
		FInterchangeUsdInfoCacheImpl& Impl,
		bool bIsInsideSkelRoot,
		int32& OutDepth,
		UE::UsdInfoCache::Private::EGeometryCachePrimState& OutState
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::RecursiveCheckForGeometryCache);

		using namespace UE::UsdInfoCache::Private;

		// With this recursive check for geometry cache, we want to find branches with an animated mesh at the leaf and find the root where they can
		// meet. This root prim will collapses the static and animated meshes under it into a single geometry cache.

		FReadScopeLock ScopeLock{Impl.InfoMapLock};
		UE::UsdInfoCache::Private::FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];

		UE::FUsdPrim UsdPrim = Context.GetUsdStage().GetPrimAtPath(Info.PrimPath);
		if (!UsdPrim)
		{
			return;
		}
		bIsInsideSkelRoot |= UsdPrim.IsA(TEXT("SkelRoot"));

		TArray<int32> Depths;
		Depths.SetNum(Info.ChildIndices.Num());

		TArray<EGeometryCachePrimState> States;
		States.SetNum(Info.ChildIndices.Num());

		const int32 MinBatchSize = 1;
		ParallelFor(
			TEXT("RecursiveCheckForGeometryCache"),
			Info.ChildIndices.Num(),
			MinBatchSize,
			[&Info, &Context, &Impl, bIsInsideSkelRoot, &Depths, &States](int32 Index)
			{
				RecursiveCheckForGeometryCache(Info.ChildIndices[Index], Context, Impl, bIsInsideSkelRoot, Depths[Index], States[Index]);
			}
		);

		// A geometry cache "branch" starts from an animated mesh prim for which we assign a depth of 0
		// Other branches, without any animated mesh, we don't care about and will remain at -1
		int32 Depth = -1;
		if (UsdUtils::IsAnimatedMesh(UsdPrim))
		{
			Depth = 0;
		}
		else
		{
			// The depth is propagated from children to parent, incremented by 1 at each level,
			// with the parent depth being the deepest of its children depth
			int32 ChildDepth = -1;
			for (int32 Index = 0; Index < Depths.Num(); ++Index)
			{
				if (Depths[Index] > -1)
				{
					ChildDepth = FMath::Max(ChildDepth, Depths[Index] + 1);
				}
			}
			Depth = ChildDepth;
		}

		// Along with the depth, we want some hints on the content of the subtree of the prim as this will tell us
		// if the prim can serve as a root and collapse its children into a GeometryCache. The sole condition for
		// being a valid root is that all the branches of the subtree are collapsible.
		EGeometryCachePrimState ChildrenState = EGeometryCachePrimState::None;
		for (EGeometryCachePrimState ChildState : States)
		{
			ChildrenState |= ChildState;
		}

		EGeometryCachePrimState PrimState = EGeometryCachePrimState::None;
		const bool bIsMesh = UsdPrim.IsA(TEXT("Mesh"));
		const bool bIsXform = UsdPrim.IsA(TEXT("Xform"));
		if (bIsMesh)
		{
			// A skinned mesh can never be considered part of a geometry cache.
			// We may run into skinned meshes that were already handled by a Skeleton handler elsewhere, and need to manually skip them
			if (GIsEditor && bIsInsideSkelRoot && UsdPrim.HasAPI(TEXT("SkelBindingAPI")))
			{
				PrimState = EGeometryCachePrimState::Uncollapsible;
			}
			else
			{
				// Animated or static mesh. Static meshes could potentially be animated by transforms in their hierarchy.
				// A mesh prim should be a leaf, but it can have GeomSubset prims as children, but those don't
				// affect the collapsibility status.
				PrimState = EGeometryCachePrimState::Mesh;
			}
		}
		else if (bIsXform)
		{
			// An xform prim is considered collapsible since it could have a mesh prim under it. It has to bubble up its children state.
			PrimState = ChildrenState != EGeometryCachePrimState::None ? ChildrenState | EGeometryCachePrimState::Xform
																	   : EGeometryCachePrimState::Xform;
		}
		else
		{
			// This prim is not considered collapsible with some exception
			// Like a Scope could have some meshes under it, so it has to bubble up its children state
			const bool bIsException = UsdPrim.IsA(TEXT("Scope"));
			if (bIsException && EnumHasAnyFlags(ChildrenState, EGeometryCachePrimState::Mesh))
			{
				PrimState = ChildrenState;
			}
			else
			{
				PrimState = EGeometryCachePrimState::Uncollapsible;
			}
		}

		// A prim could be a potential root if it has a reference or payload to an allowed file type for GeometryCache
		bool bIsPotentialRoot = false;
		{
			TArray<FString> ReferencedFiles = UsdUtils::GetReferenceOrPayloadFilePaths(UsdPrim);
			for (const FString& FilePath : ReferencedFiles)
			{
				FString Extension = FPaths::GetExtension(FilePath);

				if (Impl.AllowedExtensionsForGeometryCacheSource.Contains(Extension))
				{
					bIsPotentialRoot = true;
					break;
				}
			}
		}

		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(Info);
			Info.GeometryCacheDepth = Depth;
			Info.GeometryCacheState = PrimState;
		}

		// We've encountered a potential root and the subtree has a geometry cache branch, so find its root
		if (bIsPotentialRoot && Depth > -1)
		{
			if (Depth > UE::UsdInfoCache::Private::GeometryCacheMaxDepth)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"DeepGeometryCache",
						"Prim '{0}' is potentially a geometry cache {1} levels deep, which exceeds the limit of {2}. "
						"This could affect its imported animation. The limit can be increased with the cvar USD.GeometryCache.MaxDepth if needed."
					),
					FText::FromString(Info.PrimPath.GetString()),
					Depth,
					UE::UsdInfoCache::Private::GeometryCacheMaxDepth
				));
			}
			FindValidGeometryCacheRoot(UsdPrim, Context, Impl, PrimState);
			Depth = -1;
		}

		OutDepth = Depth;
		OutState = PrimState;
	}

	void CheckForGeometryCache(UInterchangeUsdContext& Context, FInterchangeUsdInfoCacheImpl& Impl)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::CheckForGeometryCache);

		if (!UE::UsdInfoCache::Private::UseGeometryCacheUSD)
		{
			return;
		}

		UE::FUsdPrim PseudoRoot = Context.GetUsdStage().GetPseudoRoot();

		// If the stage doesn't contain any animated mesh prims, then don't bother doing a full check
		bool bHasAnimatedMesh = false;
		{
			for (const UE::FUsdPrim& ChildPrim : UsdUtils::GetAllPrimsOfType(PseudoRoot, TEXT("UsdGeomMesh")))
			{
				if (UsdUtils::IsAnimatedMesh(ChildPrim))
				{
					bHasAnimatedMesh = true;
					break;
				}
			}
		}
		if (!bHasAnimatedMesh)
		{
			return;
		}

		int32 Depth = -1;
		EGeometryCachePrimState State = EGeometryCachePrimState::None;
		const bool bIsInsideSkelRoot = false;
		RecursiveCheckForGeometryCache(0, Context, Impl, bIsInsideSkelRoot, Depth, State);

		// If we end up with a positive depth, it means the check found an animated mesh somewhere
		// but no potential root before reaching the pseudoroot, so find one
		if (Depth > -1)
		{
			if (Depth > UE::UsdInfoCache::Private::GeometryCacheMaxDepth)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"DeepGeometryCacheInStage",
						"The stage has a geometry cache {0} levels deep, which exceeds the limit of {1}. "
						"This could affect its imported animation. The limit can be increased with the cvar USD.GeometryCache.MaxDepth if needed."
					),
					Depth,
					UE::UsdInfoCache::Private::GeometryCacheMaxDepth
				));
			}

			// The pseudoroot itself cannot be a root for the geometry cache so start from its children
			const bool bTraverseInstanceProxies = true;
			for (const UE::FUsdPrim& Child : PseudoRoot.GetFilteredChildren(bTraverseInstanceProxies))
			{
				FindValidGeometryCacheRoot(Child, Context, Impl, State);
			}
		}
	}

	void DebugOutput(FInterchangeUsdInfoCacheImpl& Impl)
	{
#if false
		FReadScopeLock ScopeLock{Impl.InfoMapLock};

		const static TMap<EGeometryCachePrimState, FString> GeometryCacheStateStrings = {
			{EGeometryCachePrimState::None, TEXT("None")},
			{EGeometryCachePrimState::Uncollapsible, TEXT("Uncollapsible")},
			{EGeometryCachePrimState::Mesh, TEXT("Mesh")},
			{EGeometryCachePrimState::Xform, TEXT("Xform")},
			{EGeometryCachePrimState::Collapsible, TEXT("Collapsible")},
			{EGeometryCachePrimState::ValidRoot, TEXT("ValidRoot")},
		};

		UE_LOGF(LogTemp, Log, "Built info cache");
		UE_LOGF(LogTemp, Log, "Format: \"[Prim name]: [Collapses subtree?], [Can subtree be collapsed?], [Geometry cache depth], [Geometry cache state]\"");

		TFunction<void(uint64, const FString&)> OutputPrim = nullptr;
		OutputPrim = [&Impl, &OutputPrim](uint64 PrimIndex, const FString& Indent)
		{
			UE::UsdInfoCache::Private::FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];

			const FString* FoundGeometryCacheStateString = GeometryCacheStateStrings.Find(Info.GeometryCacheState);

			UE_LOGF(
				LogTemp,
				Log,
				"%ls%ls: %ls, %ls, %d, %ls",
				*Indent,
				*Info.PrimPath.GetName(),
				Info.bCollapsesChildren ? TEXT("yes") : TEXT("no"),
				Info.bXformSubtreeCanBeCollapsed.IsSet() ? Info.bXformSubtreeCanBeCollapsed.GetValue() ? TEXT("yes") : TEXT("no") : TEXT("unknown"),
				Info.GeometryCacheDepth,
				FoundGeometryCacheStateString != nullptr ? **FoundGeometryCacheStateString : TEXT("unknown")
			);

			FString ChildIndent = Indent + TEXT("\t");
			for (uint64 ChildIndex : Info.ChildIndices)
			{
				OutputPrim(ChildIndex, ChildIndent);
			}
		};

		const uint64 RootIndex = 0;
		const FString EmptyIndent = TEXT("\t");
		OutputPrim(RootIndex, EmptyIndent);
#endif
	}
}	 // namespace UE::UsdInfoCache::Private

FInterchangeUsdInfoCache::FInterchangeUsdInfoCache()
{
	using namespace UE::UsdInfoCache::Private;

	Impl = MakePimpl<FInterchangeUsdInfoCacheImpl>();
}

FInterchangeUsdInfoCache::~FInterchangeUsdInfoCache()
{
}

bool FInterchangeUsdInfoCache::ContainsInfoAboutPrim(const UE::FSdfPath& Path) const
{
	using namespace UE::UsdInfoCache::Private;

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		if (ImplPtr->bIsBuilding)
		{
			ensureMsgf(
				false,
				TEXT(
					"The FInterchangeUsdInfoCache should not be queried during the build process! Make sure that your schema handler does not access the info cache from its CanBeCollapsed() and CollapsesChildren() functions. The system will automatically traverse and cache subtree results."
				)
			);
			return false;
		}

		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);
		return ImplPtr->InfoMap.Contains(Path);
	}

	return false;
}

bool FInterchangeUsdInfoCache::IsPathCollapsed(const UE::FSdfPath& Path) const
{
	using namespace UE::UsdInfoCache::Private;

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		if (ImplPtr->bIsBuilding)
		{
			ensureMsgf(
				false,
				TEXT(
					"The FInterchangeUsdInfoCache should not be queried during the build process! Make sure that your schema handler does not access the info cache from its CanBeCollapsed() and CollapsesChildren() functions. The system will automatically traverse and cache subtree results."
				)
			);
			return false;
		}

		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			uint64 IterIndex = INDEX_NONE;
			{
				// We're only collapsed if a parent collapses us
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];
				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				IterIndex = Info.ParentInfoIndex;
			}

			while (IterIndex != INDEX_NONE)
			{
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[IterIndex];
				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (Info.bCollapsesChildren)
				{
					return true;
				}

				IterIndex = Info.ParentInfoIndex;
			}

			return false;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return false;
}

bool FInterchangeUsdInfoCache::DoesPathCollapseChildren(const UE::FSdfPath& Path) const
{
	using namespace UE::UsdInfoCache::Private;

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		if (ImplPtr->bIsBuilding)
		{
			ensureMsgf(
				false,
				TEXT(
					"The FInterchangeUsdInfoCache should not be queried during the build process! Make sure that your schema handler does not access the info cache from its CanBeCollapsed() and CollapsesChildren() functions. The system will automatically traverse and cache subtree results."
				)
			);
			return false;
		}

		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			uint64 IterIndex = INDEX_NONE;
			{
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];

				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (!Info.bCollapsesChildren)
				{
					// If this prim doesn't even want to collapse its children, we're done
					return false;
				}

				IterIndex = Info.ParentInfoIndex;
			}

			// Even if this prim wants to collapse its children though, it could be that it's collapsed
			// by a parent instead (collapsing is always done top-down)
			while (IterIndex != INDEX_NONE)
			{
				const FUsdPrimInfo& AncestorInfo = ImplPtr->PrimInfoArray[IterIndex];

				FReadScopeLock PrimLock = ImplPtr->LockForReading(AncestorInfo);
				if (AncestorInfo.bCollapsesChildren)
				{
					return false;
				}

				IterIndex = AncestorInfo.ParentInfoIndex;
			}

			return true;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return false;
}

UE::FSdfPath FInterchangeUsdInfoCache::UnwindToNonCollapsedPath(const UE::FSdfPath& Path) const
{
	using namespace UE::UsdInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::UnwindToNonCollapsedPath);

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		if (ImplPtr->bIsBuilding)
		{
			ensureMsgf(
				false,
				TEXT(
					"The FInterchangeUsdInfoCache should not be queried during the build process! Make sure that your schema handler does not access the info cache from its CanBeCollapsed() and CollapsesChildren() functions. The system will automatically traverse and cache subtree results."
				)
			);
			return {};
		}

		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			TFunction<TOptional<UE::FSdfPath>(uint64)> GetCollapseRootFromParent;
			GetCollapseRootFromParent = [ImplPtr, &GetCollapseRootFromParent](uint64 Index) -> TOptional<UE::FSdfPath>
			{
				if (Index == INDEX_NONE)
				{
					return {};
				}

				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[Index];

				uint64 ParentIndex = INDEX_NONE;
				{
					FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
					ParentIndex = Info.ParentInfoIndex;
				}

				TOptional<UE::FSdfPath> CollapseRootPath = GetCollapseRootFromParent(ParentIndex);
				if (CollapseRootPath.IsSet())
				{
					// Our parent says it is collapsed with this collapse root: That's going to be
					// the collapse root for our children too
					return CollapseRootPath;
				}

				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (Info.bCollapsesChildren)
				{
					// We are the collapse root, let's return this to our children
					return Info.PrimPath;
				}

				// Nothing collapses so far
				return {};
			};

			TOptional<UE::FSdfPath> CollapseRoot = GetCollapseRootFromParent(*PrimIndex);
			if (CollapseRoot.IsSet())
			{
				return CollapseRoot.GetValue();
			}

			// We're not being collapsed by anything, so we're already the "non collapsed path"
			const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];
			FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
			return Info.PrimPath;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return Path;
}

void FInterchangeUsdInfoCache::Build(UInterchangeUsdContext& Context)
{
	using namespace UE::UsdInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::Build);

	// Read cvars once before we start
	{
		static IConsoleVariable* GeometryCacheEnableCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.GeometryCache.Enable"));
		UseGeometryCacheUSD = GeometryCacheEnableCVar && GeometryCacheEnableCVar->GetBool();

		static IConsoleVariable* GeometryCacheDepthCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.GeometryCache.MaxDepth"));
		GeometryCacheMaxDepth = GeometryCacheDepthCVar ? GeometryCacheDepthCVar->GetInt() : DEFAULT_GEOMETRY_CACHE_MAX_DEPTH;

		static IConsoleVariable* NumLocksCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.NumPerPrimLocks"));
		NumPerPrimLocks = FMath::Max(NumLocksCvar ? NumLocksCvar->GetInt() : DEFAULT_NUM_PER_PRIM_LOCKS, 1);
	}

	Clear();

	UE::FUsdStage Stage = Context.GetUsdStage();
	if (!Stage)
	{
		return;
	}

	FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	if (ImplPtr->bIsBuilding)
	{
		ensureMsgf(
			false,
			TEXT("FInterchangeUsdInfoCache::Build() should not be called in the scope of another FInterchangeUsdInfoCache::Build() call!")
		);
		return;
	}

	{
		TGuardValue<bool> Guard{ImplPtr->bIsBuilding, true};

		// Do this after we potentially got a new value for NumPrimLocks from the cvar
		ImplPtr->ReallocatePrimLocks();

		PreallocateForSubtree(UE::FSdfPath::AbsoluteRootPath(), Stage, *ImplPtr);

		CheckForGeometryCache(Context, *ImplPtr);

		const uint64 PseudoRootIndex = 0;
		RecursiveQueryCollapsesChildren(PseudoRootIndex, Context, *ImplPtr);
	}

	DebugOutput(*ImplPtr);
}

void FInterchangeUsdInfoCache::Clear()
{
	using namespace UE::UsdInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::Clear);

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		if (ImplPtr->bIsBuilding)
		{
			ensureMsgf(
				false,
				TEXT("FInterchangeUsdInfoCache::Clear() should not be called in the scope of a FInterchangeUsdInfoCache::Build() call!")
			);
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeUsdInfoCache::Clear::InfoMapEmpty);
		FWriteScopeLock ScopeLock(ImplPtr->InfoMapLock);
		ImplPtr->PrimInfoArray.Empty();
		ImplPtr->InfoMap.Empty();
	}
}

bool FInterchangeUsdInfoCache::IsEmpty()
{
	using namespace UE::UsdInfoCache::Private;

	if (FInterchangeUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);
		return ImplPtr->InfoMap.IsEmpty();
	}

	return true;
}

#undef DEFAULT_GEOMETRY_CACHE_MAX_DEPTH
#undef DEFAULT_NUM_PER_PRIM_LOCKS
#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
