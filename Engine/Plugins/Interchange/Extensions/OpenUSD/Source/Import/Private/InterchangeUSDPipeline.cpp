// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUSDPipeline.h"

#include "InterchangeImportModule.h"
#include "InterchangeGeometryCacheFactoryNode.h"
#include "InterchangeHeterogeneousVolumeActorFactoryNode.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeSparseVolumeTextureFactoryNode.h"
#include "InterchangeVolumeNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Parameterization/PatchBasedMeshUVGenerator.h"
#include "Sampling/MeshMapBaker.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "StaticMeshAttributes.h"

#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "UnrealUSDWrapper.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "UsdWrappers/SdfPath.h"
#include "USDMaterialUtils.h"

#if WITH_EDITORONLY_DATA
#include "AssetUtils/Texture2DBuilder.h"
#endif	  // WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeUSDPipeline)

#define LOCTEXT_NAMESPACE "USDPipeline"

DEFINE_LOG_CATEGORY_STATIC(LogInterchangeUSDPipeline, Log, All);

namespace UE::InterchangeUsdPipeline::Private
{
#if USE_USD_SDK
	const FString& GetPseudoRootTranslatedNodeUid()
	{
		const static FString Result = UE::Interchange::USD::MakeNodeUid(UE::FSdfPath::AbsoluteRootPath().GetString());
		return Result;
	}

	const FString& GetPseudoRootFactoryNodeUid()
	{
		const static FString Result = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(GetPseudoRootTranslatedNodeUid());
		return Result;
	}

	// Moves our primvar-compatible custom attributes from our flagged material instance translated nodes to the
	// corresponding factory nodes.
	//
	// Must run before ProcessMeshNodes(), as it will use these attributes
	void MoveCustomAttributesToMaterialFactoryNodes(UInterchangeBaseNodeContainer& NodeContainer)
	{
		NodeContainer.IterateNodesOfType<UInterchangeMaterialInstanceFactoryNode>(
			[&NodeContainer](const FString& NodeUid, UInterchangeMaterialInstanceFactoryNode* FactoryNode)
			{
				TArray<FString> TargetNodeUids;
				FactoryNode->GetTargetNodeUids(TargetNodeUids);

				for (const FString& TargetNodeUid : TargetNodeUids)
				{
					const UInterchangeMaterialInstanceNode* TranslatedNode = Cast<UInterchangeMaterialInstanceNode>(
						NodeContainer.GetNode(TargetNodeUid)
					);

					if (!TranslatedNode)
					{
						continue;
					}

					// We explicitly flag the material nodes to parse with this attribute
					bool bParse;
					if (!TranslatedNode->GetBooleanAttribute(*UE::Interchange::USD::ParseMaterialIdentifier, bParse) || !bParse)
					{
						continue;
					}

					TArray<UE::Interchange::FAttributeKey> AllAttributeKeys;
					TranslatedNode->GetAttributeKeys(AllAttributeKeys);

					for (const UE::Interchange::FAttributeKey& AttributeKey : AllAttributeKeys)
					{
						const FString AttributeKeyString = AttributeKey.ToString();

						// It's another attribute related to primvar-compatible materials, just move it over to the factory node as-is
						if (AttributeKeyString.StartsWith(UE::Interchange::USD::ParameterToPrimvarAttributePrefix))
						{
							FString AttributeValue;
							if (TranslatedNode->GetStringAttribute(AttributeKey.Key, AttributeValue))
							{
								FactoryNode->AddStringAttribute(AttributeKeyString, AttributeValue);
							}
						}
						else if (AttributeKeyString.StartsWith(UE::Interchange::USD::PrimvarUVIndexAttributePrefix))
						{
							int32 AttributeValue;
							if (TranslatedNode->GetInt32Attribute(AttributeKey.Key, AttributeValue))
							{
								FactoryNode->AddInt32Attribute(AttributeKeyString, AttributeValue);
							}
						}
					}
				}
			}
		);
	}

	// Move custom attribute info from first volume nodes into corresponding factory nodes.
	// This info originally came from the USD codeless SparseVolumeTextureAPI schema.
	void MoveCustomAttributesToVolumeFactoryNode(const UInterchangeVolumeNode* VolumeNode, UInterchangeSparseVolumeTextureFactoryNode* FactoryNode)
	{
		using namespace UE::Interchange;
		using namespace UE::Interchange::USD;

		using StringSetterFunc = decltype(&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelX);
		using EnumSetterFunc = decltype(&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAFormat);

		const static TMap<FAttributeKey, StringSetterFunc> StringSetters{
			{FAttributeKey{SparseVolumeTexture::AttributesAChannelR}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelX},
			{FAttributeKey{SparseVolumeTexture::AttributesAChannelG}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelY},
			{FAttributeKey{SparseVolumeTexture::AttributesAChannelB}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelZ},
			{FAttributeKey{SparseVolumeTexture::AttributesAChannelA}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelW},
			{FAttributeKey{SparseVolumeTexture::AttributesBChannelR}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelX},
			{FAttributeKey{SparseVolumeTexture::AttributesBChannelG}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelY},
			{FAttributeKey{SparseVolumeTexture::AttributesBChannelB}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelZ},
			{FAttributeKey{SparseVolumeTexture::AttributesBChannelA}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelW},
		};

		const static TMap<FAttributeKey, EnumSetterFunc> EnumSetters{
			{FAttributeKey{SparseVolumeTexture::AttributesAFormat}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAFormat},
			{FAttributeKey{SparseVolumeTexture::AttributesBFormat}, &UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBFormat},
		};

		TArray<FAttributeKey> AttributeKeys;
		VolumeNode->GetAttributeKeys(AttributeKeys);

		for (const FAttributeKey& AttributeKey : AttributeKeys)
		{
			if (StringSetterFunc FoundStringSetter = StringSetters.FindRef(AttributeKey))
			{
				FString Value;
				if (VolumeNode->GetStringAttribute(AttributeKey.ToString(), Value))
				{
					(FactoryNode->*FoundStringSetter)(Value);
				}
			}
			else if (EnumSetterFunc FoundEnumSetter = EnumSetters.FindRef(AttributeKey))
			{
				int32 Value = 0;
				if (VolumeNode->GetInt32Attribute(AttributeKey.ToString(), Value))
				{
					(FactoryNode->*FoundEnumSetter)(static_cast<EInterchangeSparseVolumeTextureFormat>(Value));
				}
			}
		}

		int32 NumberPrimvar = 0;
		VolumeNode->GetInt32Attribute(Primvar::Number, NumberPrimvar);
		FactoryNode->AddInt32Attribute(Primvar::Number, NumberPrimvar);
		for (int32 Index = 0; Index < NumberPrimvar; ++Index)
		{
			FString PrimvarName;
			FString Attribute = Primvar::ShaderNodeSparseVolumeTextureSample + FString::FromInt(Index);
			VolumeNode->GetStringAttribute(Attribute, PrimvarName);
			FactoryNode->AddStringAttribute(Attribute, PrimvarName);
		}
	}

	void ProcessHeterogeneousVolumeSceneNode(const UInterchangeSceneNode* SceneNode, UInterchangeBaseNodeContainer& NodeContainer)
	{
		// Find the corresponding actor factory node
		UInterchangeHeterogeneousVolumeActorFactoryNode* ActorFactoryNode = nullptr;
		{
			TArray<FString> TargetNodeUids;
			SceneNode->GetTargetNodeUids(TargetNodeUids);

			for (const FString& TargetNodeUid : TargetNodeUids)
			{
				UInterchangeFactoryBaseNode* FactoryBaseNode = NodeContainer.GetFactoryNode(TargetNodeUid);
				ActorFactoryNode = Cast<UInterchangeHeterogeneousVolumeActorFactoryNode>(FactoryBaseNode);
				if (ActorFactoryNode)
				{
					break;
				}
			}
		}

		// Find the material it is using
		UInterchangeMaterialInstanceFactoryNode* MaterialFactoryNode = nullptr;
		{
			TMap<FString, FString> OutMaterialDependencies;
			SceneNode->GetSlotMaterialDependencies(OutMaterialDependencies);

			if (FString* MaterialInstanceUid = OutMaterialDependencies.Find(UE::Interchange::Volume::VolumetricMaterial))
			{
				const FString MaterialFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(*MaterialInstanceUid);
				MaterialFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(NodeContainer.GetFactoryNode(MaterialFactoryNodeUid));
			}
		}
		if (!MaterialFactoryNode)
		{
			return;
		}

		// Make sure we spawn our actor after the material has finished generating, as we need to assign the
		// material to the actor
		if (ActorFactoryNode && MaterialFactoryNode)
		{
			ActorFactoryNode->AddFactoryDependencyUid(MaterialFactoryNode->GetUniqueID());
		}

		// Fixup the SVT assignment on the material, if needed
		//
		// Reference: CollectMaterialParameterTextureAssignment from USDVolVolumeTranslator.cpp
		// We don't refactor/reuse the function as the logic/data is slightly different
		//
		// The USDTranslator has already translated into string attributes the best SVT to material parameter assignment
		// we could come up with so far, which includes the fallback of interpreting the field names as material parameter names.
		// In here, we check if it used that fallback, and if so we check if it will work or not, and if not we use another fallback.
		// The intent here is to make sure that we get *something* assigned to the SVT material, like the legacy schema translator does
		{
			TMap<FString, FString> CleanAttributeNamesToVolumes;
			bool bIsFallbackCase = false;

			TArray<UE::Interchange::FAttributeKey> AllAttributeKeys;
			MaterialFactoryNode->GetAttributeKeys(AllAttributeKeys);
			for (const UE::Interchange::FAttributeKey& AttributeKey : AllAttributeKeys)
			{
				const UE::Interchange::EAttributeTypes AttributeType = MaterialFactoryNode->GetAttributeType(AttributeKey);
				if (AttributeType != UE::Interchange::EAttributeTypes::String)
				{
					continue;
				}

				const FString AttributeKeyString = AttributeKey.ToString();
				if (!AttributeKeyString.Contains(UE::Interchange::USD::VolumeFieldNameMaterialParameterPrefix))
				{
					continue;
				}

				bIsFallbackCase = true;

				FString AttributeValue;
				if (!MaterialFactoryNode->GetStringAttribute(AttributeKeyString, AttributeValue))
				{
					continue;
				}

				// e.g. go from "Inputs:USD_FieldName_density:Value" to "USD_FieldName_density"
				FString CleanAttributeName = UInterchangeShaderPortsAPI::MakeInputName(AttributeKeyString);

				// e.g. go "USD_FieldName_density" to "density"
				CleanAttributeName.RemoveFromStart(UE::Interchange::USD::VolumeFieldNameMaterialParameterPrefix);

				CleanAttributeNamesToVolumes.Add(CleanAttributeName, AttributeValue);

				// We definitely don't want the fallback attrs to continue past this pipeline, as they have no meaning for Interchange itself
				ensure(MaterialFactoryNode->RemoveAttribute(AttributeKeyString));
			}
			if (!bIsFallbackCase)
			{
				// If the translator didn't emit any of these VolumeFieldNameMaterialParameterPrefix attributes then we're not
				// in a fallback material assignment case and can just stop now
				return;
			}

			// Get the actual SVT parameter names from the material we're using on this actor
			TArray<FString> SparseVolumeTextureParameterNames;
			{
				FString ParentMaterialContentPath;
				if (!MaterialFactoryNode->GetCustomParent(ParentMaterialContentPath))
				{
					return;
				}

				FSoftObjectPath ReferencedObject{ParentMaterialContentPath};

				// This is why this entire pipeline needs to be on the game thread
				UMaterial* ParentMaterial = nullptr;
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(ReferencedObject.TryLoad()))
				{
					ParentMaterial = MaterialInterface->GetMaterial();
				}
				if (!ParentMaterial)
				{
					return;
				}

				SparseVolumeTextureParameterNames = UsdUtils::GetSparseVolumeTextureParameterNames(ParentMaterial);
			}

			// Compensate for how parameter names are usually upper case in UE and field names are lower case in USD
			TMap<FString, FString> LowercaseParamNamesToOriginal;
			LowercaseParamNamesToOriginal.Reserve(SparseVolumeTextureParameterNames.Num());
			for (const FString& ParamName : SparseVolumeTextureParameterNames)
			{
				LowercaseParamNamesToOriginal.Add(ParamName.ToLower(), ParamName);
			}

			// Check if our volume prim field names actually match material parameters (if we disregard casing)
			bool bFoundFieldNameMatch = false;
			for (const TPair<FString, FString>& AttrPair : CleanAttributeNamesToVolumes)
			{
				const FString& CleanAttributeName = AttrPair.Key;
				const FString& VolumeUid = AttrPair.Value;

				if (FString* FoundParamName = LowercaseParamNamesToOriginal.Find(CleanAttributeName.ToLower()))
				{
					bFoundFieldNameMatch = true;

					const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputValueKey(*FoundParamName);
					MaterialFactoryNode->AddStringAttribute(ParameterKey, VolumeUid);
				}
			}
			if (bFoundFieldNameMatch)
			{
				// If we found any kind of match with the field names let's just take that
				return;
			}

			// As a final fallback case just assign SVTs to the material parameters in alphabetical order like the legacy
			// schema translator does

			TArray<FString> VolumeUids;
			{
				TSet<FString> UniqueVolumeIds;
				UniqueVolumeIds.Reserve(CleanAttributeNamesToVolumes.Num());
				for (const TPair<FString, FString>& ParamPair : CleanAttributeNamesToVolumes)
				{
					UniqueVolumeIds.Add(ParamPair.Value);
				}
				VolumeUids = UniqueVolumeIds.Array();
			}

			SparseVolumeTextureParameterNames.Sort();
			VolumeUids.Sort();

			for (int32 Index = 0; Index < VolumeUids.Num() && Index < SparseVolumeTextureParameterNames.Num(); ++Index)
			{
				const FString& VolumeUid = VolumeUids[Index];
				const FString& ParameterName = SparseVolumeTextureParameterNames[Index];
				const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName);

				MaterialFactoryNode->AddStringAttribute(ParameterKey, VolumeUid);
			}
		}
	}

	void ProcessVolumeNodes(UInterchangeBaseNodeContainer& NodeContainer)
	{
		using namespace UE::Interchange;
		using namespace UE::Interchange::USD;

		TSet<const UInterchangeSceneNode*> VolumeSceneNodes;
		TMap<const UInterchangeVolumeNode*, UInterchangeSparseVolumeTextureFactoryNode*> FirstVolumeToFactoryNode;

		NodeContainer.IterateNodes(
			[&NodeContainer, &FirstVolumeToFactoryNode, &VolumeSceneNodes](const FString& FactoryNodeUid, UInterchangeBaseNode* BaseNode)
			{
				if (UInterchangeSparseVolumeTextureFactoryNode* VolumeFactoryNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(BaseNode))
				{
					FString FirstVolumeUid = UInterchangeFactoryBaseNode::BuildTranslatedNodeUid(FactoryNodeUid);

					const UInterchangeVolumeNode* FirstVolume = Cast<const UInterchangeVolumeNode>(NodeContainer.GetNode(FirstVolumeUid));
					ensure(FirstVolume);

					FirstVolumeToFactoryNode.Add(FirstVolume, VolumeFactoryNode);
				}
				else if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
				{
					TArray<FString> TargetAssetUids;
					SceneNode->GetTargetNodeUids(TargetAssetUids);

					for (const FString& TargetUid : TargetAssetUids)
					{
						if (const UInterchangeVolumeNode* Volume = Cast<const UInterchangeVolumeNode>(NodeContainer.GetNode(TargetUid)))
						{
							VolumeSceneNodes.Add(SceneNode);
							break;
						}
					}
				}
			}
		);

		for (TPair<const UInterchangeVolumeNode*, UInterchangeSparseVolumeTextureFactoryNode*> Pair : FirstVolumeToFactoryNode)
		{
			const UInterchangeVolumeNode* VolumeNode = Pair.Key;
			UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = Pair.Value;
			MoveCustomAttributesToVolumeFactoryNode(VolumeNode, FactoryNode);
		}

		for (const UInterchangeSceneNode* SceneNode : VolumeSceneNodes)
		{
			ProcessHeterogeneousVolumeSceneNode(SceneNode, NodeContainer);
		}
	}

	void RemovePseudoRootFactoryNode(UInterchangeBaseNodeContainer& NodeContainer)
	{
		const FString& FactoryNodeUid = GetPseudoRootFactoryNodeUid();
		UInterchangeFactoryBaseNode* PseudoRootFactoryNode = NodeContainer.GetFactoryNode(FactoryNodeUid);
		if (PseudoRootFactoryNode)
		{
			// Move all root node children to top level
			for (const FString& ChildNodeUid : NodeContainer.GetNodeChildrenUids(FactoryNodeUid))
			{
				if (UInterchangeFactoryBaseNode* ChildNode = NodeContainer.GetFactoryNode(ChildNodeUid))
				{
					NodeContainer.ClearNodeParentUid(ChildNodeUid);
				}
			}

			// Disable the pseudoroot itself (we shouldn't have any asset node for it though)
			NodeContainer.RemoveFactoryNode(FactoryNodeUid);
		}
	}

	/**
	 * Collects all UInterchangeMeshNode instances that contribute to a given factory node.
	 * For non-collapsed meshes this is typically a single mesh node. For collapsed meshes (via MeshBundleNode),
	 * this returns all constituent mesh nodes in the bundle.
	 */
	void CollectContributingMeshNodes(
		const UInterchangeFactoryBaseNode* FactoryNode,
		const UInterchangeBaseNodeContainer* NodeContainer,
		TArray<const UInterchangeMeshNode*>& OutMeshNodes
	)
	{
		TArray<FString> TargetNodeUids;
		FactoryNode->GetTargetNodeUids(TargetNodeUids);

		for (const FString& TargetUid : TargetNodeUids)
		{
			const UInterchangeBaseNode* TargetNode = NodeContainer->GetNode(TargetUid);
			if (!TargetNode)
			{
				continue;
			}

			if (const UInterchangeMeshNode* MeshNode = Cast<const UInterchangeMeshNode>(TargetNode))
			{
				OutMeshNodes.Add(MeshNode);
			}
			else if (const UInterchangeMeshBundleNode* BundleNode = Cast<const UInterchangeMeshBundleNode>(TargetNode))
			{
				TArray<FString> BundleMeshNodeUids;
				BundleNode->GetMeshNodeUids(BundleMeshNodeUids);
				for (const FString& MeshNodeUid : BundleMeshNodeUids)
				{
					if (const UInterchangeMeshNode* InnerMeshNode = Cast<const UInterchangeMeshNode>(NodeContainer->GetNode(MeshNodeUid)))
					{
						OutMeshNodes.Add(InnerMeshNode);
					}
				}
			}
		}
	}

	/**
	 * Helper: tries to read a user-defined attribute of type T from a mesh node.
	 * Returns true if the attribute was authored and its value is placed in OutValue.
	 */
	template<typename T>
	bool TryGetBuildSettingUserAttribute(const UInterchangeMeshNode* MeshNode, const FString& AttributeName, T& OutValue)
	{
		TOptional<FString> PayloadKey;
		return UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(MeshNode, AttributeName, OutValue, PayloadKey);
	}

	/**
	 * Finds the first mesh node in the array that has any user-defined attribute with the given schema prefix
	 * (e.g. "unreal:meshBuild:" or "unreal:nanite:"). Uses a single probe attribute to check.
	 * If multiple mesh nodes have the schema, emits a divergence warning and returns the first one found.
	 */
	const UInterchangeMeshNode* FindMeshNodeWithSchema(
		const TArray<const UInterchangeMeshNode*>& MeshNodes,
		const FString& ProbeAttributeName,
		const TCHAR* SchemaDisplayName
	)
	{
		const UInterchangeMeshNode* FoundNode = nullptr;

		for (const UInterchangeMeshNode* MeshNode : MeshNodes)
		{
			// Check if this mesh node has any attribute from this schema by probing one known attribute.
			// We use GetUserDefinedAttribute_FString to check for existence of any type without needing the exact type.
			TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos;
			UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(MeshNode, AttributeInfos);

			bool bHasSchemaAttribute = false;
			for (const FInterchangeUserDefinedAttributeInfo& Info : AttributeInfos)
			{
				if (Info.Name.StartsWith(ProbeAttributeName))
				{
					bHasSchemaAttribute = true;
					break;
				}
			}

			if (bHasSchemaAttribute)
			{
				if (!FoundNode)
				{
					FoundNode = MeshNode;
				}
				else
				{
					UE_LOG(
						LogInterchangeUSDPipeline,
						Warning,
						TEXT("Multiple mesh prims have %s applied when collapsing into a single static mesh. Using settings from the first prim encountered."),
						SchemaDisplayName
					);
					return FoundNode;
				}
			}
		}

		return FoundNode;
	}

	/**
	 * Transfers StaticMeshBuildSettingsAPI and NaniteBuildSettingsAPI user attributes from mesh nodes
	 * to the corresponding factory node custom attributes.
	 *
	 * For each schema, finds the first contributing mesh node that has it applied and uses all
	 * values from that node. If multiple mesh nodes have the same schema, a warning is emitted
	 * and the first node's values are used.
	 */
	void TransferBuildSettingsToFactoryNodes(UInterchangeBaseNodeContainer& NodeContainer)
	{
		NodeContainer.IterateNodesOfType<UInterchangeStaticMeshFactoryNode>(
			[&NodeContainer](const FString& NodeUid, UInterchangeStaticMeshFactoryNode* FactoryNode)
			{
				TArray<const UInterchangeMeshNode*> MeshNodes;
				CollectContributingMeshNodes(FactoryNode, &NodeContainer, MeshNodes);

				if (MeshNodes.IsEmpty())
				{
					return;
				}

				// StaticMeshBuildSettingsAPI: find the first mesh node with this schema
				if (const UInterchangeMeshNode* BuildSettingsNode = FindMeshNodeWithSchema(MeshNodes, TEXT("unreal:meshBuild:"), *UnrealIdentifiers::StaticMeshBuildSettingsAPI))
				{
					bool BoolValue;
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:useMikkTSpace"), BoolValue))
					{
						FactoryNode->SetCustomUseMikkTSpace(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:recomputeNormals"), BoolValue))
					{
						FactoryNode->SetCustomRecomputeNormals(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:recomputeTangents"), BoolValue))
					{
						FactoryNode->SetCustomRecomputeTangents(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:computeWeightedNormals"), BoolValue))
					{
						FactoryNode->SetCustomComputeWeightedNormals(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:removeDegenerates"), BoolValue))
					{
						FactoryNode->SetCustomRemoveDegenerates(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:useHighPrecisionTangentBasis"), BoolValue))
					{
						FactoryNode->SetCustomUseHighPrecisionTangentBasis(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:useFullPrecisionUVs"), BoolValue))
					{
						FactoryNode->SetCustomUseFullPrecisionUVs(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:buildReversedIndexBuffer"), BoolValue))
					{
						FactoryNode->SetCustomBuildReversedIndexBuffer(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:generateLightmapUVs"), BoolValue))
					{
						FactoryNode->SetCustomGenerateLightmapUVs(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:generateDistanceFieldAsIfTwoSided"), BoolValue))
					{
						FactoryNode->SetCustomGenerateDistanceFieldAsIfTwoSided(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:supportFaceRemap"), BoolValue))
					{
						FactoryNode->SetCustomSupportFaceRemap(BoolValue);
					}

					int32 Int32Value;
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:minLightmapResolution"), Int32Value))
					{
						FactoryNode->SetCustomMinLightmapResolution(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:srcLightmapIndex"), Int32Value))
					{
						FactoryNode->SetCustomSrcLightmapIndex(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:dstLightmapIndex"), Int32Value))
					{
						FactoryNode->SetCustomDstLightmapIndex(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:maxLumenMeshCards"), Int32Value))
					{
						FactoryNode->SetCustomMaxLumenMeshCards(Int32Value);
					}

					float FloatValue;
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:distanceFieldResolutionScale"), FloatValue))
					{
						FactoryNode->SetCustomDistanceFieldResolutionScale(FloatValue);
					}

					// buildScale3D is a float3 in USD, stored as FVector3f user attribute
					FVector3f Vector3fValue;
					if (TryGetBuildSettingUserAttribute(BuildSettingsNode, TEXT("unreal:meshBuild:buildScale3D"), Vector3fValue))
					{
						FactoryNode->SetCustomBuildScale3D(FVector(Vector3fValue));
					}
				}

				// NaniteBuildSettingsAPI: find the first mesh node with this schema
				if (const UInterchangeMeshNode* NaniteSettingsNode = FindMeshNodeWithSchema(MeshNodes, TEXT("unreal:nanite:"), *UnrealIdentifiers::NaniteBuildSettingsAPI))
				{
					bool BoolValue;
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:enabled"), BoolValue))
					{
						FactoryNode->SetCustomBuildNanite(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:explicitTangents"), BoolValue))
					{
						FactoryNode->SetCustomNaniteExplicitTangents(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:lerpUVs"), BoolValue))
					{
						FactoryNode->SetCustomNaniteLerpUVs(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:separable"), BoolValue))
					{
						FactoryNode->SetCustomNaniteSeparable(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:voxelNDF"), BoolValue))
					{
						FactoryNode->SetCustomNaniteVoxelNDF(BoolValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:voxelOpacity"), BoolValue))
					{
						FactoryNode->SetCustomNaniteVoxelOpacity(BoolValue);
					}

					int32 Int32Value;
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:positionPrecision"), Int32Value))
					{
						FactoryNode->SetCustomNanitePositionPrecision(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:normalPrecision"), Int32Value))
					{
						FactoryNode->SetCustomNaniteNormalPrecision(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:tangentPrecision"), Int32Value))
					{
						FactoryNode->SetCustomNaniteTangentPrecision(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:boneWeightPrecision"), Int32Value))
					{
						FactoryNode->SetCustomNaniteBoneWeightPrecision(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:targetMinimumResidencyInKB"), Int32Value))
					{
						FactoryNode->SetCustomNaniteTargetMinimumResidencyInKB(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:numRays"), Int32Value))
					{
						FactoryNode->SetCustomNaniteNumRays(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:voxelLevel"), Int32Value))
					{
						FactoryNode->SetCustomNaniteVoxelLevel(Int32Value);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:displacementUVChannel"), Int32Value))
					{
						FactoryNode->SetCustomNaniteDisplacementUVChannel(Int32Value);
					}

					float FloatValue;
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:keepPercentTriangles"), FloatValue))
					{
						FactoryNode->SetCustomNaniteKeepPercentTriangles(FloatValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:trimRelativeError"), FloatValue))
					{
						FactoryNode->SetCustomNaniteTrimRelativeError(FloatValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:fallbackPercentTriangles"), FloatValue))
					{
						FactoryNode->SetCustomNaniteFallbackPercentTriangles(FloatValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:fallbackRelativeError"), FloatValue))
					{
						FactoryNode->SetCustomNaniteFallbackRelativeError(FloatValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:maxEdgeLengthFactor"), FloatValue))
					{
						FactoryNode->SetCustomNaniteMaxEdgeLengthFactor(FloatValue);
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:rayBackUp"), FloatValue))
					{
						FactoryNode->SetCustomNaniteRayBackUp(FloatValue);
					}

					// Token/enum attributes: stored as FName by the USD type system, convert to int32 enum values
					FName TokenValue;
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:shapePreservation"), TokenValue))
					{
						if (TokenValue == TEXT("None"))
						{
							FactoryNode->SetCustomNaniteShapePreservation((int32)EInterchangeNaniteShapePreservation::None);
						}
						else if (TokenValue == TEXT("PreserveArea"))
						{
							FactoryNode->SetCustomNaniteShapePreservation((int32)EInterchangeNaniteShapePreservation::PreserveArea);
						}
						else if (TokenValue == TEXT("Voxelize"))
						{
							FactoryNode->SetCustomNaniteShapePreservation((int32)EInterchangeNaniteShapePreservation::Voxelize);
						}
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:generateFallback"), TokenValue))
					{
						if (TokenValue == TEXT("PlatformDefault"))
						{
							FactoryNode->SetCustomNaniteGenerateFallback((int32)EInterchangeNaniteGenerateFallback::PlatformDefault);
						}
						else if (TokenValue == TEXT("Enabled"))
						{
							FactoryNode->SetCustomNaniteGenerateFallback((int32)EInterchangeNaniteGenerateFallback::Enabled);
						}
					}
					if (TryGetBuildSettingUserAttribute(NaniteSettingsNode, TEXT("unreal:nanite:fallbackTarget"), TokenValue))
					{
						if (TokenValue == TEXT("Auto"))
						{
							FactoryNode->SetCustomNaniteFallbackTarget((int32)EInterchangeNaniteFallbackTarget::Auto);
						}
						else if (TokenValue == TEXT("PercentTriangles"))
						{
							FactoryNode->SetCustomNaniteFallbackTarget((int32)EInterchangeNaniteFallbackTarget::PercentTriangles);
						}
						else if (TokenValue == TEXT("RelativeError"))
						{
							FactoryNode->SetCustomNaniteFallbackTarget((int32)EInterchangeNaniteFallbackTarget::RelativeError);
						}
					}
				}
			}
		);
	}

#if WITH_EDITORONLY_DATA
	/**
	 * Add the primvar attributes from the mesh nodes to factory nodes
	 * Also add the array names as payload key attributes to filter them in the Translator
	 */
	void AddPrimvarAttributesToFactoryNodes(
		const UInterchangeMeshNode* MeshNode,
		UInterchangeMeshFactoryNode* MeshFactoryNode,
		UInterchangeBaseNodeContainer* NodeContainer
	)
	{
		using namespace UE::Interchange;

		if (int32 NumPrimvar; MeshNode->GetInt32Attribute(USD::Primvar::Number, NumPrimvar))
		{
			MeshFactoryNode->AddInt32Attribute(USD::Primvar::Number, NumPrimvar);
			MeshFactoryNode->AddPayloadKeyInt32Attribute(USD::Primvar::Number, NumPrimvar);
			for (int32 Index = 0; Index < NumPrimvar; ++Index)
			{
				FString PrimvarAttribute = USD::Primvar::Name + FString::FromInt(Index);
				if (FString PrimvarName; MeshNode->GetStringAttribute(PrimvarAttribute, PrimvarName))
				{
					MeshFactoryNode->AddStringAttribute(PrimvarAttribute, PrimvarName);
					MeshFactoryNode->AddPayloadKeyStringAttribute(PrimvarAttribute, PrimvarName);
				}

				if (bool bTangentSpace; MeshNode->GetBooleanAttribute(PrimvarAttribute, bTangentSpace))
				{
					MeshFactoryNode->AddBooleanAttribute(PrimvarAttribute, bTangentSpace);
				}

				FString ShaderNodeTextureSampleAttribute = USD::Primvar::ShaderNodeTextureSample + FString::FromInt(Index);
				if (FString ShaderNodeTextureSampleUID; MeshNode->GetStringAttribute(ShaderNodeTextureSampleAttribute, ShaderNodeTextureSampleUID))
				{
					// At this stage the USD pipeline should be the latest one in the stack, we should retrieve the MaterialExpression
					// Factory Node associated to this ShaderNode
					FString TextureSampleFactoryNodeUID = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ShaderNodeTextureSampleUID);

					if (Cast<UInterchangeMaterialExpressionFactoryNode>(NodeContainer->GetFactoryNode(TextureSampleFactoryNodeUID)))
					{
						MeshFactoryNode->AddStringAttribute(ShaderNodeTextureSampleAttribute, TextureSampleFactoryNodeUID);
					}
				}
			}
		}
	}
#endif	  // WITH_EDITORONLY_DATA

	struct FMaterialPrimvarInfo
	{
		TMap<FString, FString> ParameterToPrimvar;
		TMap<FString, int32> PrimvarToUVIndex;
	};

	// Gets or creates a filled out FMaterialPrimvarInfo struct for a particular factory node.
	// We use this because we may iterate over the same material many times during ProcessMeshNodes,
	// and we don't want to recompute this info every time
	FMaterialPrimvarInfo* GetOrCreateMaterialPrimvarInfo(
		UInterchangeMaterialInstanceFactoryNode* InstanceFactoryNode,
		TMap<FString, FMaterialPrimvarInfo>& InOutFactoryNodeUidToInfo
	)
	{
		using namespace UE::Interchange;

		if (!InstanceFactoryNode)
		{
			return nullptr;
		}

		const FString& MaterialUid = InstanceFactoryNode->GetUniqueID();

		FMaterialPrimvarInfo* FoundInfo = InOutFactoryNodeUidToInfo.Find(MaterialUid);
		if (!FoundInfo)
		{
			FMaterialPrimvarInfo& NewInfo = InOutFactoryNodeUidToInfo.Emplace(MaterialUid);

			TArray<UE::Interchange::FAttributeKey> MaterialAttributeKeys;
			InstanceFactoryNode->GetAttributeKeys(MaterialAttributeKeys);

			NewInfo.ParameterToPrimvar.Reserve(MaterialAttributeKeys.Num());
			NewInfo.PrimvarToUVIndex.Reserve(MaterialAttributeKeys.Num());
			for (const UE::Interchange::FAttributeKey& AttributeKey : MaterialAttributeKeys)
			{
				const FString& AttributeKeyString = AttributeKey.ToString();

				if (AttributeKeyString.StartsWith(USD::ParameterToPrimvarAttributePrefix))
				{
					FString ParameterName = AttributeKeyString;
					ensure(ParameterName.RemoveFromStart(USD::ParameterToPrimvarAttributePrefix));

					FString Primvar;
					if (InstanceFactoryNode->GetStringAttribute(AttributeKeyString, Primvar))
					{
						NewInfo.ParameterToPrimvar.Add(ParameterName, Primvar);
					}
				}
				else if (AttributeKeyString.StartsWith(USD::PrimvarUVIndexAttributePrefix))
				{
					FString ParameterName = AttributeKeyString;
					ensure(ParameterName.RemoveFromStart(USD::PrimvarUVIndexAttributePrefix));

					int32 UVIndex = -1;
					if (InstanceFactoryNode->GetInt32Attribute(AttributeKeyString, UVIndex))
					{
						NewInfo.PrimvarToUVIndex.Add(ParameterName, UVIndex);
					}
				}
			}

			FoundInfo = &NewInfo;
		}

		return FoundInfo;
	}

	// Returns a primvar-compatible version of OriginalFactoryNode.
	// i.e. creates (if needed) a new MaterialInstanceFactoryNode that has a primvar to UV index
	// mapping that is usable with the primvar to UV index layout in MeshPrimvarToUVIndex
	UInterchangeMaterialInstanceFactoryNode* GetOrCreateCompatibleMaterial(
		const UInterchangeMaterialInstanceFactoryNode* OriginalFactoryNode,
		const FMaterialPrimvarInfo* OriginalMaterialPrimvarInfo,
		const TMap<FString, int32>& MeshPrimvarToUVIndex,
		UInterchangeBaseNodeContainer* InNodeContainer
	)
	{
		using namespace UE::Interchange;

		if (!OriginalFactoryNode || !OriginalMaterialPrimvarInfo || !InNodeContainer)
		{
			return nullptr;
		}

		// First, let's create the target primvar UVIndex assignment that is compatible with this mesh.
		// We use an array of TPairs here so that we can sort these into a deterministic order for hashing later.
		TArray<TPair<FString, int32>> CompatiblePrimvarAndUVIndexPairs;
		CompatiblePrimvarAndUVIndexPairs.Reserve(OriginalMaterialPrimvarInfo->PrimvarToUVIndex.Num());
		for (const TPair<FString, int32>& Pair : OriginalMaterialPrimvarInfo->PrimvarToUVIndex)
		{
			const FString& MaterialPrimvar = Pair.Key;

			bool bFoundUVIndex = false;

			// Mesh has this primvar available at some UV index, point to it
			if (const int32* FoundMeshUVIndex = MeshPrimvarToUVIndex.Find(MaterialPrimvar))
			{
				int32 MeshUVIndex = *FoundMeshUVIndex;
				if (MeshUVIndex >= 0 && MeshUVIndex < USD_PREVIEW_SURFACE_MAX_UV_SETS)
				{
					CompatiblePrimvarAndUVIndexPairs.Add(TPair<FString, int32>{MaterialPrimvar, MeshUVIndex});
					bFoundUVIndex = true;
				}
			}

			if (!bFoundUVIndex)
			{
				// Point this primvar to read an unused UV index instead, since our mesh doesn't have this primvar
				CompatiblePrimvarAndUVIndexPairs.Add(TPair<FString, int32>{MaterialPrimvar, UNUSED_UV_INDEX});
			}
		}

		// Generate a deterministic hash based on the original material hash and this primvar UVIndex assignment
		CompatiblePrimvarAndUVIndexPairs.Sort(
			[](const TPair<FString, int32>& LHS, const TPair<FString, int32>& RHS)
			{
				if (LHS.Key == RHS.Key)
				{
					return LHS.Value < RHS.Value;
				}
				else
				{
					return LHS.Key < RHS.Key;
				}
			}
		);
		FSHAHash Hash;
		FSHA1 SHA1;
		const FString OriginalNodeUid = OriginalFactoryNode->GetUniqueID();
		SHA1.UpdateWithString(*OriginalNodeUid, OriginalNodeUid.Len());
		for (const TPair<FString, int32>& Pair : CompatiblePrimvarAndUVIndexPairs)
		{
			SHA1.UpdateWithString(*Pair.Key, Pair.Key.Len());
			SHA1.Update((const uint8*)&Pair.Value, sizeof(Pair.Value));
		}
		SHA1.Final();
		SHA1.GetHash(&Hash.Hash[0]);

		const FString CompatibleNodeUid = OriginalNodeUid + UE::Interchange::USD::CompatibleMaterialUidSuffix + Hash.ToString();

		// Check if we made this compatible material before
		UInterchangeMaterialInstanceFactoryNode* CompatibleFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(
			InNodeContainer->GetFactoryNode(CompatibleNodeUid)
		);
		if (CompatibleFactoryNode)
		{
			return CompatibleFactoryNode;
		}

		// We need to actually create a new compatible material
		CompatibleFactoryNode = NewObject<UInterchangeMaterialInstanceFactoryNode>(InNodeContainer);

		UObject* Object = nullptr;
		CompatibleFactoryNode->CopyWithObject(OriginalFactoryNode, Object);

		InNodeContainer->SetupNode(
			CompatibleFactoryNode,
			CompatibleNodeUid,
			OriginalFactoryNode->GetDisplayLabel() + TEXT("_Compatible"),
			OriginalFactoryNode->GetNodeContainerType());
		
		// Cleanup our old primvar to UV Index mapping in case we don't overlap perfectly
		for (const TPair<FString, int32>& OldPair : OriginalMaterialPrimvarInfo->PrimvarToUVIndex)
		{
			const FString& Primvar = OldPair.Key;
			const FString AttributeKeyString = USD::PrimvarUVIndexAttributePrefix + Primvar;

			CompatibleFactoryNode->RemoveAttribute(AttributeKeyString);
		}

		// We also need to move our pairs into an actual TMap now, as we'll need to query into it later
		TMap<FString, int32> CompatiblePrimvarToUVIndex;
		CompatiblePrimvarToUVIndex.Reserve(CompatiblePrimvarAndUVIndexPairs.Num());

		// Set our new primvar to UV index mapping on the factory node (only for the sake of information / debugging)
		for (const TPair<FString, int32>& NewPair : CompatiblePrimvarAndUVIndexPairs)
		{
			const FString& Primvar = NewPair.Key;
			const int32 UVIndex = NewPair.Value;
			const FString AttributeKeyString = USD::PrimvarUVIndexAttributePrefix + Primvar;

			CompatibleFactoryNode->AddInt32Attribute(AttributeKeyString, UVIndex);

			CompatiblePrimvarToUVIndex.Add(Primvar, UVIndex);
		}

		// Set our compatible UV indices into the actual material parameter attributes that will be set
		// on the material instance
		for (const TPair<FString, FString>& ParameterPair : OriginalMaterialPrimvarInfo->ParameterToPrimvar)
		{
			const FString& MaterialParameter = ParameterPair.Key;
			const FString& Primvar = ParameterPair.Value;

			int32 UVIndex = UNUSED_UV_INDEX;
			if (int32* FoundUVIndex = CompatiblePrimvarToUVIndex.Find(Primvar))
			{
				UVIndex = *FoundUVIndex;
			}

			const bool bEnableTexture = UVIndex >= 0 && UVIndex < UNUSED_UV_INDEX;
			const FString EnableKey = UInterchangeShaderPortsAPI::MakeInputValueKey(
				USD::UseTextureParameterPrefix + MaterialParameter + USD::UseTextureParameterSuffix
			);
			ensure(CompatibleFactoryNode->AddFloatAttribute(EnableKey, bEnableTexture));

			const FString UVIndexKey = UInterchangeShaderPortsAPI::MakeInputValueKey(MaterialParameter + USD::UVIndexParameterSuffix);
			if (bEnableTexture)
			{
				ensure(CompatibleFactoryNode->AddFloatAttribute(UVIndexKey, UVIndex));
			}
			else
			{
				ensure(CompatibleFactoryNode->RemoveAttribute(UVIndexKey));
			}
		}

		return CompatibleFactoryNode;
	}

	// Makes sure that all materials in the MeshNode's SlotDependencies array are primvar-compatible
	// with the primvar to UV index mapping of the mesh, creating new materials and reassigning them to the
	// SlotDependencies custom attribute if needed.
	template<typename NodeType>
	void GeneratePrimvarCompatibleMaterials(
		const UInterchangeMeshNode* InMeshNode,
		NodeType* InMeshFactoryNode,
		UInterchangeBaseNodeContainer* InNodeContainer,
		TMap<FString, FMaterialPrimvarInfo>& InOutFactoryNodeUidToInfo
	)
	{
		using namespace UE::Interchange;

		// TODO: This doesn't work yet, and would produce a ton of extra broken "compatible" materials if unchecked.
		// The root issue being that the PrimvarToUVIndex map made for the Mesh prim doesn't consider how the mesh data
		// will be combined into a single Static/SkeletalMesh asset. Different Mesh prims may claim different mappings,
		// and it could be that none match the final PrimvarToUVIndex layout we actually end up with...
		//
		// Even more concerning: For now, Interchange currently will combine meshes without caring about their PrimvarToUVIndex
		// layout in general, so two cubes may have different primvars assigned to UV0, that end up merged together.
		// Now a material that wants to read this primvar will read unwanted data on half the combined mesh! On legacy USD
		// we did a pre-pass to organize which primvar belonged to which UV index in the combined mesh. On Interchange
		// this doesn't make sense, as the combining is dictated by (potentially custom) pipelines that will run only later
		if (InMeshNode->IsSkinnedMesh())
		{
			return;
		}

		// Rebuild the mesh's primvar to UV index map
		TMap<FString, int32> MeshPrimvarToUVIndex;
		{
			TArray<UE::Interchange::FAttributeKey> MeshAttributeKeys;
			InMeshNode->GetAttributeKeys(MeshAttributeKeys);

			MeshPrimvarToUVIndex.Reserve(MeshAttributeKeys.Num());

			for (const UE::Interchange::FAttributeKey& AttributeKey : MeshAttributeKeys)
			{
				const FString& AttributeKeyString = AttributeKey.ToString();

				if (AttributeKeyString.StartsWith(USD::PrimvarUVIndexAttributePrefix))
				{
					FString ParameterName = AttributeKeyString;
					ensure(ParameterName.RemoveFromStart(USD::PrimvarUVIndexAttributePrefix));

					int32 UVIndex = -1;
					if (InMeshNode->GetInt32Attribute(AttributeKeyString, UVIndex))
					{
						MeshPrimvarToUVIndex.Add(ParameterName, UVIndex);
					}
				}
			}
		}

		// Build the reverse map too. This because we can still consider a mesh and material "compatible"
		// if the material tries reading "st2" and the mesh doesn't have "st2" anywhere. We will just turn off
		// usage of that particular texture on the material instance.
		//
		// For that case to still count as "compatible" though, then whatever UV index the material is trying to
		// read "st2" from must not be used by any other primvar: Otherwise the mesh will still be providing
		// some primvar data to the material, but the material will treat it as if it were "st2" when it is not
		TArray<TSet<FString>> UVIndexToMeshPrimvars;
		UVIndexToMeshPrimvars.SetNum(USD_PREVIEW_SURFACE_MAX_UV_SETS);
		for (const TPair<FString, int32>& MeshPair : MeshPrimvarToUVIndex)
		{
			if (UVIndexToMeshPrimvars.IsValidIndex(MeshPair.Value))
			{
				UVIndexToMeshPrimvars[MeshPair.Value].Add(MeshPair.Key);
			}
		}

		// Use primvar to UV index information to construct primvar-compatible materials if needed
		TMap<FString, FString> SlotNameToMaterialFactoryNodeUid;
		InMeshFactoryNode->GetSlotMaterialDependencies(SlotNameToMaterialFactoryNodeUid);
		for (const TPair<FString, FString>& SlotPair : SlotNameToMaterialFactoryNodeUid)
		{
			const FString& SlotName = SlotPair.Key;
			const FString& MaterialFactoryNodeUid = SlotPair.Value;

			UInterchangeMaterialInstanceFactoryNode* OriginalMaterial = Cast<UInterchangeMaterialInstanceFactoryNode>(
				InNodeContainer->GetFactoryNode(MaterialFactoryNodeUid)
			);
			if (!OriginalMaterial)
			{
				continue;
			}

			FMaterialPrimvarInfo* OriginalMaterialInfo = GetOrCreateMaterialPrimvarInfo(OriginalMaterial, InOutFactoryNodeUidToInfo);
			if (!OriginalMaterialInfo)
			{
				continue;
			}

			// Check if the material's primvar-UVIndex mapping matches the mesh
			bool bCompatible = true;
			for (const TPair<FString, int32>& Pair : OriginalMaterialInfo->PrimvarToUVIndex)
			{
				const FString& MaterialPrimvar = Pair.Key;
				int32 MaterialUVIndex = Pair.Value;

				// If the mesh has the same primvar the material wants, it should be at the same UVIndex the material
				// will read from
				if (const int32* MeshUVIndex = MeshPrimvarToUVIndex.Find(MaterialPrimvar))
				{
					if (*MeshUVIndex != MaterialUVIndex)
					{
						bCompatible = false;
						break;
					}
				}

				// If the material is going to read from a given UVIndex that exists on the mesh, that UV set should
				// contain the primvar data that the material expects to read
				if (UVIndexToMeshPrimvars.IsValidIndex(MaterialUVIndex))
				{
					const TSet<FString>& CompatiblePrimvars = UVIndexToMeshPrimvars[MaterialUVIndex];
					if (!CompatiblePrimvars.Contains(MaterialPrimvar))
					{
						bCompatible = false;
						break;
					}
				}
			}

			if (!bCompatible)
			{
				// Generate a primvar-compatible material
				UInterchangeMaterialInstanceFactoryNode* CompatibleMaterial = GetOrCreateCompatibleMaterial(	//
					OriginalMaterial,
					OriginalMaterialInfo,
					MeshPrimvarToUVIndex,
					InNodeContainer
				);
				if (CompatibleMaterial)
				{
					InMeshFactoryNode->SetSlotMaterialDependencyUid(SlotName, CompatibleMaterial->GetUniqueID());
				}
			}
		}
	}

	void ProcessMeshNodes(
		UInterchangeBaseNodeContainer* NodeContainer,
		bool bGenerateCompatibleMaterials,
		EInterchangeUsdPrimvar ImportPrimvar,
		int32 SubdivisionLevel
	)
	{
		using namespace UE::Interchange;

		// Collect the nodes in a separate container because GeneratePrimvarCompatibleMaterials may add additional nodes
		// to the node container itself, which we can't do while we iterate over it
		TMap<const UInterchangeMeshNode*, UInterchangeMeshFactoryNode*> MeshNodeToMeshFactoryNode;
		TMap<const UInterchangeMeshNode*, UInterchangeMeshActorFactoryNode*> MeshNodeToMeshActorFactoryNode;

		NodeContainer->IterateNodes(
			[NodeContainer,
			 &MeshNodeToMeshFactoryNode,
			 ImportPrimvar,
			 SubdivisionLevel,
			 bGenerateCompatibleMaterials,
			 &MeshNodeToMeshActorFactoryNode](const FString& NodeUid, UInterchangeBaseNode* BaseNode)
			{
				if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode))
				{
					const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeUid);
					UInterchangeMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(NodeContainer->GetFactoryNode(FactoryNodeUid));
					if (!MeshFactoryNode)
					{
						return;
					}

					if (bGenerateCompatibleMaterials)
					{
						MeshNodeToMeshFactoryNode.Add(MeshNode, MeshFactoryNode);
					}

					if (SubdivisionLevel != 0)
					{
						MeshFactoryNode->AddPayloadKeyInt32Attribute(UE::Interchange::USD::SubdivisionLevelAttributeKey, SubdivisionLevel);
					}

#if WITH_EDITORONLY_DATA
					MeshFactoryNode->AddPayloadKeyInt32Attribute(UE::Interchange::USD::Primvar::Import, int32(ImportPrimvar));
					if (ImportPrimvar != EInterchangeUsdPrimvar::Standard)
					{
						AddPrimvarAttributesToFactoryNodes(MeshNode, MeshFactoryNode, NodeContainer);
					}
#endif	  // WITH_EDITORONLY_DATA
				}
				else if (UInterchangeGeometryCacheFactoryNode* GeometryCacheNode = Cast<UInterchangeGeometryCacheFactoryNode>(BaseNode))
				{
					if (SubdivisionLevel != 0) 
					{
						GeometryCacheNode->AddPayloadKeyInt32Attribute(UE::Interchange::USD::SubdivisionLevelAttributeKey, SubdivisionLevel);
					}
				}
				else if (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(BaseNode))
				{
					// Also search through our mesh actor factory nodes: We may have added material overrides that also need to be made compatible.
					if (bGenerateCompatibleMaterials)
					{
						TMap<FString, FString> MaterialDependencies;
						MeshActorFactoryNode->GetSlotMaterialDependencies(MaterialDependencies);
						if (MaterialDependencies.Num() == 0)
						{
							return;
						}

						FString MeshFactoryNodeUid;
						MeshActorFactoryNode->GetCustomInstancedAssetFactoryNodeUid(MeshFactoryNodeUid);
						UInterchangeMeshFactoryNode* AssignedMeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(
							NodeContainer->GetFactoryNode(MeshFactoryNodeUid)
						);
						if (!AssignedMeshFactoryNode)
						{
							return;
						}

						TArray<FString> MeshFactoryNodeTargetUids;
						AssignedMeshFactoryNode->GetTargetNodeUids(MeshFactoryNodeTargetUids);
						for (const FString& TargetUid : MeshFactoryNodeTargetUids)
						{
							if (const UInterchangeMeshNode* AssignedMeshNode = Cast<const UInterchangeMeshNode>(NodeContainer->GetNode(TargetUid)))
							{
								MeshNodeToMeshActorFactoryNode.Add(AssignedMeshNode, MeshActorFactoryNode);
							}
						}
					}
				}
			}
		);

		if (bGenerateCompatibleMaterials)
		{
			// Build this map to prevent us from reparsing the same material in case multiple meshes are using them
			TMap<FString, FMaterialPrimvarInfo> MaterialFactoryNodeUidToInfo;

			for (const TPair<const UInterchangeMeshNode*, UInterchangeMeshFactoryNode*>& MeshPair : MeshNodeToMeshFactoryNode)
			{
				GeneratePrimvarCompatibleMaterials(MeshPair.Key, MeshPair.Value, NodeContainer, MaterialFactoryNodeUidToInfo);
			}
			for (const TPair<const UInterchangeMeshNode*, UInterchangeMeshActorFactoryNode*>& ActorPair : MeshNodeToMeshActorFactoryNode)
			{
				GeneratePrimvarCompatibleMaterials(ActorPair.Key, ActorPair.Value, NodeContainer, MaterialFactoryNodeUidToInfo);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	class FUsdPrimvarBaker
	{
	public:

		FUsdPrimvarBaker(
			UStaticMesh* StaticMeshAsset,
			const UInterchangeBaseNodeContainer* NodeContainer,
			const UInterchangeFactoryBaseNode* FactoryNode
		)
			: NodeContainer{NodeContainer}
			, FactoryNode{FactoryNode}
			, StaticMeshAsset{StaticMeshAsset}
		{
		}

		template<typename T>
		struct TVertexInstanceAttributesRefGuard
		{
			TVertexInstanceAttributesRefGuard(TVertexInstanceAttributesRef<T>& InAttributesRef)
				: AttributesRef(InAttributesRef)
			{
				// store the original values
				StoredAttributes.Reserve(InAttributesRef.GetNumElements());
				for (int32 Index = 0; Index < InAttributesRef.GetNumElements(); ++Index)
				{
					StoredAttributes.Emplace(Index, AttributesRef.Get(Index));
				}
			}

			~TVertexInstanceAttributesRefGuard()
			{
				Reset();
			}

			/** Reset a TVertexInstanceAttributesRef to its previous state with its previous values */
			void Reset()
			{
				for (const TPair<FVertexInstanceID, T> Pair : StoredAttributes)
				{
					AttributesRef[Pair.Key] = Pair.Value;
				}
			}

		private:
			TVertexInstanceAttributesRef<T>& AttributesRef;
			TMap<FVertexInstanceID, T> StoredAttributes;
		};

		using FVertexInstanceAttributesNormalsRefGuard = TVertexInstanceAttributesRefGuard<FVector3f>;
		using FVertexInstanceAttributesVertexColorsRefGuard = TVertexInstanceAttributesRefGuard<FVector4f>;

		void Bake()
		{
			using namespace UE::Interchange;
			using namespace UE::Interchange::Materials;
			using namespace UE::Geometry;

			if (int32 NumberOfPrimvar; FactoryNode->GetInt32Attribute(USD::Primvar::Number, NumberOfPrimvar) && NumberOfPrimvar)
			{
				FMeshDescription MeshDescriptionPrimvar = CreateMeshDescription();

				// DynamicMesh will hold the new UV set, at the end of the process we have to convert back to the MeshDescription, so it also has the new UV set
				FDynamicMesh3 DynamicMesh;

				FStaticMeshAttributes MeshAttributes(MeshDescriptionPrimvar);
				TVertexInstanceAttributesRef<FVector4f> VertexColors = MeshAttributes.GetVertexInstanceColors();
				TVertexInstanceAttributesRef<FVector3f> Normals = MeshAttributes.GetVertexInstanceNormals();

				// we have to safe guard the vertex colors and the normals because we're directly working on them when baking them
				// so we can restore them at the end
				// Because the Conversion goes like this:
				// 1. MeshDescriptionPrimvar alters its VertexColor/Normal attribute with the stored primvar
				// 2. Conversion to DynamicMesh
				// 3. Bake
				// 4. Converting back DynamicMesh to Original MeshDescription with now the newly added UV set
				// Note: the different conversion from/to DynamicMesh/MeshDescription doesn't preserve the primvars though since they are custom vertex instance attributes
				FVertexInstanceAttributesVertexColorsRefGuard VertexColorsGuard{ VertexColors };
				FVertexInstanceAttributesNormalsRefGuard NormalsGuard{ Normals };

				bool bConverted = false;
				for (int32 Index = 0; Index < NumberOfPrimvar; ++Index)
				{
					UMaterialExpressionTextureSample* TextureSampleExpression = GetMaterialExpressionTextureSample(Index);

					if (!TextureSampleExpression)
					{
						continue;
					}

					FString PrimvarName;
					bool bTangentSpace = false;
					FactoryNode->GetStringAttribute(USD::Primvar::Name + FString::FromInt(Index), PrimvarName);
					FactoryNode->GetBooleanAttribute(USD::Primvar::TangentSpace + FString::FromInt(Index), bTangentSpace);

					MeshDescriptionPrimvar.VertexInstanceAttributes().ForEach(
					[this, &PrimvarName, &VertexColors, &Normals, TextureSampleExpression, &MeshDescriptionPrimvar, &DynamicMesh, bTangentSpace, &bConverted]
					(const FName AttributeName, auto AttributesConstRef)
					{
						if (AttributeName != PrimvarName)
						{
							return;
						}

						if (!(AttributesConstRef.IsValid() && AttributesConstRef.GetNumElements() == VertexColors.GetNumElements()
							&& AttributesConstRef.GetNumElements() == Normals.GetNumElements()))
						{
							return;
						}

						// We put the primvar in the Color/Normal channel, that way during the conversion to a DynamicMesh it will also handle the
						// non-manifold case
						for (const int32 VertexInstanceID : MeshDescriptionPrimvar.VertexInstances().GetElementIDs())
						{
							auto VertexInstanceValue = AttributesConstRef.Get(VertexInstanceID);

							// underlying type of the Value in the AttributesConstRef
							using T = std::decay_t<decltype(VertexInstanceValue)>;

							if constexpr (std::is_same_v<T, bool> || std::is_same_v<T, int32> || std::is_same_v<T, float>)
							{
								// store the value inside a float to avoid the compile error: conversion from 'AttributeType' to 'T' requires a narrowing conversion
								const float VertexInstanceValueFloat = VertexInstanceValue;
								VertexColors.Set(VertexInstanceID, FVector4f{ VertexInstanceValueFloat, VertexInstanceValueFloat, VertexInstanceValueFloat, VertexInstanceValueFloat });
							}
							else if constexpr (std::is_same_v<T, FVector2f>)
							{
								VertexColors.Set(VertexInstanceID, FVector4f{ VertexInstanceValue, VertexInstanceValue });
							}
							else if constexpr (std::is_same_v<T, FVector3f> || std::is_same_v<T, FVector4f>)
							{
								bTangentSpace
									? Normals.Set(VertexInstanceID, VertexInstanceValue)
									: VertexColors.Set(VertexInstanceID, VertexInstanceValue);
							}
						}

						FMeshDescriptionToDynamicMesh Convert;
						Convert.Convert(&MeshDescriptionPrimvar, DynamicMesh);
						DynamicMesh.Attributes()->SetNumUVLayers(UVChannel + 1);
						FDynamicMeshUVOverlay* UVOVerlay = DynamicMesh.Attributes()->GetUVLayer(UVChannel);

						FPatchBasedMeshUVGenerator Generator;
						Generator.AutoComputeUVs(DynamicMesh, *UVOVerlay);

						const FDynamicMeshAABBTree3 DetailSpatial(&DynamicMesh);
						FMeshBakerDynamicMeshSampler DetailSampler(&DynamicMesh, &DetailSpatial);
						TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
						PropertyEval->Property = bTangentSpace ? EMeshPropertyMapType::Normal : EMeshPropertyMapType::VertexColor;

						constexpr int32 TextureSize = 1024;
						const FImageDimensions ImageDimensions(TextureSize, TextureSize);
						FMeshMapBaker Baker;
						Baker.SetCorrespondenceStrategy(FMeshBaseBaker::ECorrespondenceStrategy::Identity);
						Baker.SetTargetMesh(&DynamicMesh);
						Baker.SetDetailSampler(&DetailSampler);
						Baker.AddEvaluator(PropertyEval);
						Baker.SetTargetMeshUVLayer(UVChannel);
						Baker.SetDimensions(ImageDimensions);
						Baker.SetProjectionDistance(3.0f);
						Baker.SetSamplesPerPixel(1);
						Baker.SetFilter(FMeshMapBaker::EBakeFilterType::BSpline);
						Baker.SetGutterEnabled(true);
						Baker.SetGutterSize(4);
						Baker.SetTileSize(TextureSize);
						Baker.Bake();

						FTexture2DBuilder TextureBuilder;
						const FTexture2DBuilder::ETextureType TextureType = GetTextureType(TextureSampleExpression);

						TextureBuilder
							.InitializeAndReplaceExistingTexture(Cast<UTexture2D>(TextureSampleExpression->Texture.Get()), TextureType, ImageDimensions);
						TextureBuilder.Copy(*Baker.GetBakeResults(0)[0]);
						TextureBuilder.Commit();

						TextureSampleExpression->ConstCoordinate = UVChannel;

						MaterialExpressionTextureSamples.Add(TextureSampleExpression);
						bConverted = true;
					});
				}

				VertexColorsGuard.Reset();
				NormalsGuard.Reset();

				if (bConverted)
				{
					FDynamicMeshToMeshDescription Converter;
					// convert back to the mesh description so it's also adding the new UV set
					// though this conversion unfortunately doesn't preserve the primvars, do we need them after all that?
					Converter.Convert(&DynamicMesh, *StaticMeshAsset->GetMeshDescription(0));
				}
				UpdateMaterials();
			}
		}

	private:

		UMaterialExpressionTextureSample* GetMaterialExpressionTextureSample(int32 IndexPrimvar) const
		{
			using namespace UE::Interchange;
			using namespace UE::Interchange::Materials;

			UMaterialExpressionTextureSample* MaterialExpressionTextureSample = nullptr;

			FString ShaderNodeTextureSampleUID;
			FactoryNode->GetStringAttribute(USD::Primvar::ShaderNodeTextureSample + FString::FromInt(IndexPrimvar), ShaderNodeTextureSampleUID);
			if (UInterchangeMaterialExpressionFactoryNode* TextureSampleExpressionFactoryNode = Cast<UInterchangeMaterialExpressionFactoryNode>(NodeContainer->GetFactoryNode(ShaderNodeTextureSampleUID)))
			{
				if (FString ExpressionPath;
					TextureSampleExpressionFactoryNode->GetStringAttribute(Factory::Expression::Path.ToString(), ExpressionPath))
				{
					MaterialExpressionTextureSample = Cast<UMaterialExpressionTextureSample>(FSoftObjectPath{ExpressionPath}.TryLoad());
				}
			}

			return MaterialExpressionTextureSample;
		}

		/** Create a mesh description based on the one from the StaticMesh, the StaticMesh will now hold a new UV set */
		FMeshDescription CreateMeshDescription()
		{
			FMeshDescription* MeshDescription = StaticMeshAsset->GetMeshDescription(0);
			FStaticMeshAttributes StaticMeshAttributes(*MeshDescription);
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();

			if (AreUVsOverlapping(VertexInstanceUVs))
			{
				UVChannel++;
			}

			FMeshDescription MeshDescriptionPrimvar(*MeshDescription);
			return MeshDescriptionPrimvar;
		}

		void UpdateMaterials() const
		{
			TSet<UMaterial*> Materials;
			for (UMaterialExpressionTextureSample* TextureSampleExpression : MaterialExpressionTextureSamples)
			{
				Materials.FindOrAdd(TextureSampleExpression->Material);
				TextureSampleExpression->PostEditChange();
				TextureSampleExpression->Modify();
			}

			for (UMaterial* Material : Materials)
			{
				Material->PostEditChange();
				Material->ForceRecompileForRendering();
				Material->MarkPackageDirty();
			}
		}

		UE::Geometry::FTexture2DBuilder::ETextureType GetTextureType(UMaterialExpressionTextureSample* TextureSampleExpression)
		{
			using namespace UE::Geometry;
			switch (TextureSampleExpression->SamplerType)
			{
				case EMaterialSamplerType::SAMPLERTYPE_Color:
					return FTexture2DBuilder::ETextureType::Color;

				case EMaterialSamplerType::SAMPLERTYPE_Normal:
					return FTexture2DBuilder::ETextureType::NormalMap;

				default:
					return FTexture2DBuilder::ETextureType::ColorLinear;
			}
		}

		struct FUVKeyFunc : public BaseKeyFuncs<FVector2f, FVector2f>
		{
			static constexpr float Threshold = 0.00001f;

			static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
			{
				return Element;
			}

			static FORCEINLINE uint32 GetKeyHash(ElementInitType& Key)
			{
				int32 RoundedU = FMath::RoundToInt(Key.X / Threshold);
				int32 RoundedV = FMath::RoundToInt(Key.Y / Threshold);
				return HashCombine(GetTypeHash(RoundedU), GetTypeHash(RoundedV));
			}

			static FORCEINLINE bool Matches(ElementInitType A, ElementInitType B)
			{
				return FMath::IsNearlyEqual(A.X, B.X, Threshold) && FMath::IsNearlyEqual(A.Y, B.Y, Threshold);
			}
		};

		bool AreUVsOverlapping(const TVertexInstanceAttributesRef<FVector2f>& UVs)
		{
			FMeshDescription* MeshDecription = StaticMeshAsset->GetMeshDescription(0);
			TSet<FVector2f, FUVKeyFunc> UVSet;

			for (const FVertexInstanceID VertexInstanceID : MeshDecription->VertexInstances().GetElementIDs())
			{
				FVector2f UV = UVs.Get(VertexInstanceID);

				if (UVSet.Contains(UV))
				{
					USD_LOG_USERWARNING(LOCTEXT("UVsOverlapping", "UVs are Overlapping"));
					return true;
				}

				UVSet.Add(UV);
			}

			return false;
		}

	private:

		const UInterchangeBaseNodeContainer* NodeContainer;
		const UInterchangeFactoryBaseNode* FactoryNode;
		UStaticMesh* StaticMeshAsset;
		TArray<UMaterialExpressionTextureSample*> MaterialExpressionTextureSamples;
		int32 UVChannel = 0;
	};
#endif	  // WITH_EDITORONLY_DATA
#endif	  // USE_USD_SDK
}	 // namespace UE::InterchangeUsdPipeline::Private

UInterchangeUsdPipeline::UInterchangeUsdPipeline()
	: SubdivisionLevel(0)
	, bImportPseudoRoot(true)
	, bGeneratePrimvarCompatibleMaterials(true)
{
}

void UInterchangeUsdPipeline::ExecutePostFactoryPipeline(
	const UInterchangeBaseNodeContainer* BaseNodeContainer,
	const FString& NodeKey,
	UObject* CreatedAsset,
	bool bIsAReimport
)
{
#if USE_USD_SDK && WITH_EDITORONLY_DATA
	if (!BaseNodeContainer || !CreatedAsset)
	{
		return;
	}

	const UInterchangeFactoryBaseNode* FactoryNode = BaseNodeContainer->GetFactoryNode(NodeKey);
	if (!FactoryNode)
	{
		return;
	}

	if (UStaticMesh* StaticMeshAsset = Cast<UStaticMesh>(CreatedAsset))
	{
		if(ImportPrimvars != EInterchangeUsdPrimvar::Standard)
		{
			using namespace UE::InterchangeUsdPipeline::Private;
			FUsdPrimvarBaker Baker(StaticMeshAsset, BaseNodeContainer, FactoryNode);
			Baker.Bake();
		}
	}

	// Assign the right SVT asset to the corresponding SVT material expression
	if (USparseVolumeTexture* SparseVolumeAsset = Cast<USparseVolumeTexture>(CreatedAsset))
	{
		using namespace UE::Interchange::USD;
		using namespace UE::Interchange::Materials;

		TArray<UMaterialExpression*> MaterialExpressions;
		TSet<UMaterial*> Materials;
		TSet<FString> VolumePrimvars; // if one of the names is the same as the attribute in the ShaderNode then we'll assign the SVT asset

		int32 Number = 0;
		FactoryNode->GetInt32Attribute(Primvar::Number, Number);
		VolumePrimvars.Reserve(Number);
		for (int32 Index = 0; Index < Number; ++Index)
		{
			FString SVTSampleName;
			FactoryNode->GetStringAttribute(Primvar::ShaderNodeSparseVolumeTextureSample + FString::FromInt(Index), SVTSampleName);
			VolumePrimvars.Emplace(SVTSampleName);
		}

		using UMaterialExpressionSVTSample = UMaterialExpressionSparseVolumeTextureSampleParameter;

		BaseNodeContainer->BreakableIterateNodesOfType<UInterchangeShaderNode>(
			[BaseNodeContainer, SparseVolumeAsset, &MaterialExpressions, &VolumePrimvars](const FString&, UInterchangeShaderNode* ShaderNode)
			{
				if (FString SparseVolumeName; ShaderNode->GetStringAttribute(Primvar::ShaderNodeSparseVolumeTextureSample, SparseVolumeName))
				{
					if (VolumePrimvars.Contains(SparseVolumeName))
					{
						if (UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode =
							Cast<UInterchangeMaterialExpressionFactoryNode>(BaseNodeContainer->GetFactoryNode(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ShaderNode->GetUniqueID()))))
						{
							FString ExpressionPath;
							MaterialExpressionFactoryNode->GetStringAttribute(Factory::Expression::Path.ToString(), ExpressionPath);

							if (UMaterialExpressionSVTSample* SparseVolumeSample =
								Cast<UMaterialExpressionSVTSample>(FSoftObjectPath{ ExpressionPath }.TryLoad()))
							{
								SparseVolumeSample->SparseVolumeTexture = SparseVolumeAsset;
								MaterialExpressions.Emplace(SparseVolumeSample);
								return true;
							}
						}
					}
				}
				return false;
			});

		// Update materials and material expressions rendering
		for (UMaterialExpression* Expression : MaterialExpressions)
		{
			Materials.FindOrAdd(Expression->Material);
			Expression->PostEditChange();
			Expression->Modify();
		}

		for (UMaterial* Material : Materials)
		{
			Material->PostEditChange();
			Material->ForceRecompileForRendering();
			Material->MarkPackageDirty();
		}
	}
#endif	  // USE_USD_SDK && WITH_EDITORONLY_DATA
}

void UInterchangeUsdPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

#if WITH_EDITOR
	static IConsoleVariable* MaxSubdivCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.Subdiv.MaxSubdivLevel"));
	const int32 MaxSubdivLevel = FMath::Max(MaxSubdivCvar ? MaxSubdivCvar->GetInt() : 6, 1);
	const FString MaxSubdivLevelString = LexToString(MaxSubdivLevel);

	if (FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UInterchangeUsdPipeline, SubdivisionLevel)))
	{
		Property->SetMetaData(TEXT("UIMax"), *MaxSubdivLevelString);
		Property->SetMetaData(TEXT("ClampMax"), *MaxSubdivLevelString);
	}
#endif	  // WITH_EDITOR
}

void UInterchangeUsdPipeline::ExecutePipeline(
	UInterchangeBaseNodeContainer* NodeContainer,
	const TArray<UInterchangeSourceData*>& InSourceDatas,
	const FString& ContentBasePath
)
{
	using namespace UE::InterchangeUsdPipeline::Private;

	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	if (!NodeContainer)
	{
		UE_LOGF(LogInterchangeUSDPipeline, Warning, "UInterchangeUSDPipeline: Cannot execute pre-import pipeline because NodeContrainer is null");
		return;
	}

	const UUsdProjectSettings* UsdProjectSettings = GetDefault<UUsdProjectSettings>();
	if (IInterchangeImportModule::Get().IsSubstrateEnabled() && UsdProjectSettings)
	{
		// Let's keep Vertex Color parent material for the moment (maybe we could instead set UseVertexColor in the parent material for Substrate?)
		TMap<FString, FString> OldToNewMaterial = {
			{UsdProjectSettings->ReferencePreviewSurfaceMaterial.ToString(), 						UsdProjectSettings->ReferenceSubstratePreviewSurfaceMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTranslucentMaterial.ToString(), 			UsdProjectSettings->ReferenceSubstratePreviewSurfaceMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTwoSidedMaterial.ToString(), 				UsdProjectSettings->ReferenceSubstratePreviewSurfaceMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial.ToString(), 	UsdProjectSettings->ReferenceSubstratePreviewSurfaceMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceVTMaterial.ToString(), 						UsdProjectSettings->ReferenceSubstratePreviewSurfaceVTMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTwoSidedVTMaterial.ToString(), 				UsdProjectSettings->ReferenceSubstratePreviewSurfaceVTMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTranslucentVTMaterial.ToString(), 			UsdProjectSettings->ReferenceSubstratePreviewSurfaceVTMaterial.ToString()},
			{UsdProjectSettings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial.ToString(), 	UsdProjectSettings->ReferenceSubstratePreviewSurfaceVTMaterial.ToString()},
		};

		NodeContainer->IterateNodesOfType<UInterchangeMaterialInstanceNode>(
			[NodeContainer, &OldToNewMaterial](const FString& NodeUID, UInterchangeMaterialInstanceNode* MaterialInstanceNode)
			{
				// We can have a parent material that comes from a ShaderGraphNode (for example a TwoSided instance created based on the ShaderGraph),
				// in that case it's not a UsdPreviewSurface
				bool bShaderGaphParent = false;
				MaterialInstanceNode->GetBooleanAttribute(UE::Interchange::USD::UsdMaterialInstanceFromShaderGraph, bShaderGaphParent);
				if (bShaderGaphParent)
				{
					return;
				}

				FString ParentPath;
				if (!MaterialInstanceNode->GetCustomParent(ParentPath))
				{
					return;
				}

				FString* NewParentPath = OldToNewMaterial.Find(ParentPath);
				if (!NewParentPath)
				{
					return;
				}

				const FString FactoryNodeUID = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeUID);
				if (UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(NodeContainer->GetFactoryNode(FactoryNodeUID)))
				{
					MaterialInstanceFactoryNode->SetCustomParent(*NewParentPath);

					if (int32 Value; MaterialInstanceNode->GetInt32Attribute(UE::Interchange::USD::UsdReferenceMaterialProperties, Value))
					{
						EUsdReferenceMaterialProperties Properties = static_cast<EUsdReferenceMaterialProperties>(Value);

						if (EnumHasAnyFlags(Properties, EUsdReferenceMaterialProperties::Translucent))
						{
							MaterialInstanceFactoryNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentGreyTransmittance);
						}

						if (EnumHasAnyFlags(Properties, EUsdReferenceMaterialProperties::TwoSided))
						{
							MaterialInstanceFactoryNode->SetCustomTwoSided(true);
						}

						FString DisplacementTexture;
						float Displacement;
						if (MaterialInstanceNode->GetFloatAttribute(TEXT("Displacement"), Displacement) ||
							MaterialInstanceNode->GetTextureParameterValue(TEXT("DisplacementTexture"), DisplacementTexture))
						{
							MaterialInstanceFactoryNode->SetCustomEnableTessellation(true);
						}
					}
				}
			});
	}

#if USE_USD_SDK
	if (!NodeContainer)
	{
		return;
	}
	MoveCustomAttributesToMaterialFactoryNodes(*NodeContainer);

	ProcessVolumeNodes(*NodeContainer);

	if (!bImportPseudoRoot)
	{
		RemovePseudoRootFactoryNode(*NodeContainer);
	}

	ProcessMeshNodes(NodeContainer, bGeneratePrimvarCompatibleMaterials, ImportPrimvars, SubdivisionLevel);

	TransferBuildSettingsToFactoryNodes(*NodeContainer);
#endif	  // USE_USD_SDK
}

bool UInterchangeUsdPipeline::CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask)
{
	// TODO: Only the volume node material handling needs to be on the game thread. Maybe that can be split into
	// a separate USDGameThreadPipeline?
	return false;
}

#undef LOCTEXT_NAMESPACE