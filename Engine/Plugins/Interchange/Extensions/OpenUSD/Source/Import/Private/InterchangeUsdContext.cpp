// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUsdContext.h"

#include "InterchangeUSDInfoCache.h"
#include "InterchangeUsdTranslator.h"
#include "Objects/USDInfoCache.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeAnimationTrackSetNode.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeUsdContext)

#define LOCTEXT_NAMESPACE "InterchangeUSDContext"

UInterchangeUsdContext::UInterchangeUsdContext()
{
}

FString UInterchangeUsdContext::MakeAssetNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix, const FString& Suffix) const
{
#if USE_USD_SDK
	return UE::Interchange::USD::MakeNodeUid(Prefix + UsdUtils::GetPrototypePrimPath(Prim).GetString() + Suffix);
#else
	return FString();
#endif
}

FString UInterchangeUsdContext::MakeSceneNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix, const FString& Suffix) const
{
#if USE_USD_SDK
	return UE::Interchange::USD::MakeNodeUid(Prefix + Prim.GetPrimPath().GetString() + Suffix);
#else
	return FString();
#endif
}

void UInterchangeUsdContext::Reset()
{
#if USE_USD_SDK
	// We should manually close all our stages right away, as the UInterchangeUsdContext object itself
	// may live for a while longer until the next GC, but we may run into weirdness if we leave
	// our stages open until then

	SubTranslators.Reset();
	CurrentTrackSet = nullptr;
	{
		FWriteScopeLock Lock{NodeUidToCachedTraversalInfoLock};
		NodeUidToCachedTraversalInfo.Reset();
	}
	{
		FWriteScopeLock Lock{PrimPathToVariantToStageLock};
		PrimPathToVariantToStage.Reset();
	}
	CachedMeshConversionOptions = {};
	CachedMaterialAssignments.Reset();
	USDZFilePath.Reset();
	DecompressedUSDZRoot.Reset();
	MaterialUidToGeomProps.Reset();
	HandledPrimInfo.Reset();
	Translator = nullptr;
	NodeContainer = nullptr;
	if (bShouldCleanUpFromStageCache && StageIdInUsdUtilsStageCache != INDEX_NONE)
	{
		UsdUtils::RemoveStageFromUsdUtilsStageCache(StageIdInUsdUtilsStageCache);
		StageIdInUsdUtilsStageCache = INDEX_NONE;
	}
	InfoCache.Reset();
#endif	  // USE_USD_SDK
}

void UInterchangeUsdContext::BeginDestroy()
{
	Reset();

	Super::BeginDestroy();
}

void UInterchangeUsdContext::Initialize(UInterchangeUSDTranslator* InTranslator, UInterchangeBaseNodeContainer* InNodeContainer)
{
#if USE_USD_SDK
	Reset();

	Translator = InTranslator;
	NodeContainer = InNodeContainer;

	SetupMeshConversionOptions();
#endif	  // USE_USD_SDK
}

int64 UInterchangeUsdContext::GetStageId() const
{
#if USE_USD_SDK
	return StageIdInUsdUtilsStageCache;
#else
	return 0;
#endif	  // USE_USD_SDK
}

void UInterchangeUsdContext::SetStageId(int64 InStageId)
{
#if USE_USD_SDK
	StageIdInUsdUtilsStageCache = InStageId;
#endif	  // USE_USD_SDK
}

UE::FUsdStage UInterchangeUsdContext::GetUsdStage() const
{
#if USE_USD_SDK
	return UsdUtils::FindUsdUtilsStageCacheStageId(StageIdInUsdUtilsStageCache);
#else
	return {};
#endif	  // USE_USD_SDK
}

bool UInterchangeUsdContext::SetUsdStage(const UE::FUsdStage& InStage)
{
#if USE_USD_SDK
	UE::FUsdStage OurStage = GetUsdStage();
	if (OurStage == InStage)
	{
		return true;
	}

	// This stage is already in the stage cache somehow. Just take its existing id,
	// but let's remember to not remove it from the stage cache whenever we're done, because
	// it wasn't us that put it there in the first place
	int64 ExistingId = UsdUtils::GetUsdUtilsStageCacheStageId(InStage);
	if (ExistingId != INDEX_NONE)
	{
		bShouldCleanUpFromStageCache = false;
		StageIdInUsdUtilsStageCache = ExistingId;
		return true;
	}

	// We're adding this stage to the stage cache, so let's make sure it's cleaned
	// up whenever we're released or else it will remain there forever
	bShouldCleanUpFromStageCache = true;
	StageIdInUsdUtilsStageCache = UsdUtils::InsertStageIntoUsdUtilsStageCache(InStage);

	return StageIdInUsdUtilsStageCache != INDEX_NONE;
#else
	return false;
#endif	  // USE_USD_SDK
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FUsdInfoCache* UInterchangeUsdContext::GetInfoCache() const
{
	return nullptr;
}

void UInterchangeUsdContext::SetExternalInfoCache(FUsdInfoCache& InInfoCache)
{
}

FUsdInfoCache* UInterchangeUsdContext::CreateOwnedInfoCache()
{
	return nullptr;
}

void UInterchangeUsdContext::BuildInfoCache()
{
}

void UInterchangeUsdContext::ReleaseInfoCache()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UInterchangeUsdContext::SetupTrackSetNode()
{
#if USE_USD_SDK
	using namespace UE::Interchange::USD;

	// For now we only want a single track set (i.e. LevelSequence) per stage.
	// TODO: One track set per layer, and add the tracks to the tracksets that correspond to layers where the opinions came from
	// (similar to LevelSequenceHelper). Then we can use UInterchangeAnimationTrackSetInstanceNode to create "subsequences"
	if (CurrentTrackSet)
	{
		return;
	}

	UInterchangeBaseNodeContainer* ThisNodeContainer = GetNodeContainer();
	if (!ThisNodeContainer)
	{
		return;
	}

	UE::FSdfLayer Layer = GetUsdStage().GetRootLayer();
	const FString AnimTrackSetNodeUid = MakeNodeUid(AnimationPrefix + Layer.GetIdentifier());
	const FString AnimTrackSetNodeDisplayName = FPaths::GetBaseFilename(Layer.GetDisplayName());	// Strip extension

	// We should only have one track set node per scene for now
	UInterchangeAnimationTrackSetNode* ExistingNode = GetExistingNode<UInterchangeAnimationTrackSetNode>(*ThisNodeContainer, AnimTrackSetNodeUid);
	if (ExistingNode)
	{
		CurrentTrackSet = ExistingNode;
		return;
	};

	UInterchangeAnimationTrackSetNode* TrackSetNode = NewObject<UInterchangeAnimationTrackSetNode>(ThisNodeContainer);
	ThisNodeContainer->SetupNode(TrackSetNode, AnimTrackSetNodeUid, AnimTrackSetNodeDisplayName, EInterchangeNodeContainerType::TranslatedAsset);

	// This ends up as the LevelSequence frame rate, so it should probably match the stage's frame rate like legacy USD does
	TrackSetNode->SetCustomFrameRate(Layer.GetFramesPerSecond());

	CurrentTrackSet = TrackSetNode;
#endif	  // USE_USD_SDK
}

void UInterchangeUsdContext::SetupInterchangeInfoCache()
{
#if USE_USD_SDK
	InfoCache = MakeUnique<FInterchangeUsdInfoCache>();
	InfoCache->Build(*this);
#endif	  // USE_USD_SDK
}

UInterchangeUSDTranslator* UInterchangeUsdContext::GetTranslator() const
{
#if USE_USD_SDK
	return Translator;
#else
	return nullptr;
#endif	  // USE_USD_SDK
}

UInterchangeUsdTranslatorSettings* UInterchangeUsdContext::GetTranslatorSettings() const
{
#if USE_USD_SDK
	if (UInterchangeUSDTranslator* TranslatorPtr = GetTranslator())
	{
		return Cast<UInterchangeUsdTranslatorSettings>(TranslatorPtr->GetSettings());
	}
#endif	  // USE_USD_SDK

	return nullptr;
}

UInterchangeResultsContainer* UInterchangeUsdContext::GetResultsContainer() const
{
#if USE_USD_SDK
	if (UInterchangeUSDTranslator* TranslatorPtr = GetTranslator())
	{
		return TranslatorPtr->Results;
	}
#endif	  // USE_USD_SDK

	return nullptr;
}

UInterchangeBaseNodeContainer* UInterchangeUsdContext::GetNodeContainer() const
{
#if USE_USD_SDK
	return NodeContainer;
#else
	return nullptr;
#endif	  // USE_USD_SDK
}

#if USE_USD_SDK
FInterchangeUsdInfoCache* UInterchangeUsdContext::GetInterchangeInfoCache() const
{
	return InfoCache.Get();
}
#endif // USE_USD_SDK

void UInterchangeUsdContext::SetupMeshConversionOptions()
{
#if USE_USD_SDK
	if (UInterchangeUsdTranslatorSettings* Settings = GetTranslatorSettings())
	{
		// TODO: Change FUsdMeshConversionOptions to not hold USD types directly, so we don't have to the conversion below everywhere.
		// We can't use UsdToUnreal::ConvertToken() here because it returns a TUsdStore, and the template instantiation created in this module doesn't
		// really do anything anyway as the module doesn't use IMPLEMENT_MODULE_USD!
		// Luckily we can get around this here because pxr::TfToken doesn't allocate on its own: At most USD makes a copy of the string, which it
		// should allocate/deallocate on its own allocator.
		CachedMeshConversionOptions.PurposesToLoad = static_cast<EUsdPurpose>(Settings->GeometryPurpose);
		CachedMeshConversionOptions.MaterialPurpose = Settings->MaterialPurpose.IsNone()
														  ? pxr::UsdShadeTokens->allPurpose
														  : pxr::TfToken{TCHAR_TO_UTF8(*Settings->MaterialPurpose.ToString())};

		// Interchange doesn't know the final render context at the time we collect material assignments
		// (we query with universalRenderContext and resolve render contexts at a later stage via schema handlers),
		// so we accept any material binding regardless of whether it provides a surface shader for the render context.
		CachedMeshConversionOptions.bForceCheckUnrealMaterialAttribute = true;
		CachedMeshConversionOptions.bRequireMaterialsHaveProvidedRenderContext = false;
	}
#endif	  // USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
