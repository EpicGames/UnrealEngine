// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeUsdTranslator.h"

#include "InterchangeOpenUSDImportModule.h"
#include "SchemaHandlers/HandlerAccumulatedInfo.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/SchemaHandlerRegistry.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "UnrealUSDWrapper.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDLayerUtils.h"
#include "USDLog.h"
#include "USDShadeConversion.h"
#include "USDStageOptions.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelBinding.h"
#include "UsdWrappers/UsdSkelBlendShapeQuery.h"
#include "UsdWrappers/UsdSkelCache.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdTyped.h"
#include "UsdWrappers/UsdVariantSets.h"

#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTranslatorHelper.h"
#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Nodes/InterchangeSourceNode.h"

#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"
#include "InterchangeResult.h"
#include "Logging/TokenizedMessage.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSection.h"
#include "UDIMUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeUsdTranslator)

#define LOCTEXT_NAMESPACE "InterchangeUSDTranslator"

static bool GInterchangeEnableUSDImport = true;
static FAutoConsoleVariableRef CVarInterchangeEnableUSDImport(
	TEXT("Interchange.FeatureFlags.Import.USD"),
	GInterchangeEnableUSDImport,
	TEXT("Whether USD support is enabled.")
);

static bool GInterchangeEnableUSDLevelImport = false;
static FAutoConsoleVariableRef CVarInterchangeEnableUSDLevelImport(
	TEXT("Interchange.FeatureFlags.Import.USD.ToLevel"),
	GInterchangeEnableUSDLevelImport,
	TEXT("Whether support for USD level import is enabled.")
);

namespace UE::InterchangeUsdTranslator::Private
{
	using namespace UE::Interchange::USD;

	class UInterchangeUSDTranslatorImpl
	{
	public:
#if USE_USD_SDK
		TArray<TSharedRef<FSchemaHandler>> SchemaHandlers;
#endif	  // USE_USD_SDK
	};

	bool DecompressUSDZFileToTempFolder(const FString& InUSDZFilePath, FString& OutDecompressedUSDZRoot)
	{
#if USE_USD_SDK
		if (!UsdUtils::IsZipArchive(InUSDZFilePath))
		{
			return false;
		}

		const FString Prefix = FPaths::GetBaseFilename(InUSDZFilePath);
		const FString EmptyExtension = TEXT("");
		const FString TempFolder = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), *Prefix, *EmptyExtension);
		FString DecompressedRoot;
		const UsdUtils::EUSDZDecompressResult DecompressResult = UsdUtils::DecompressUSDZFile(InUSDZFilePath, TempFolder, &DecompressedRoot);
		if (UsdUtils::EUSDZDecompressResult::Failed == DecompressResult)
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("FailedToDecompressUSDZ", "Failed to decompress USDZ file '{0}'."),
				FText::FromString(InUSDZFilePath)
			));

			return false;
		}

		OutDecompressedUSDZRoot = DecompressedRoot;

		ensure(!OutDecompressedUSDZRoot.IsEmpty());

		if (UsdUtils::EUSDZDecompressResult::SuccessWithSkippedEntries == DecompressResult)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailedToFullyDecompressUSDZ", "Failed to fully decompress USDZ file '{0}': Textures may not be handled correctly."),
				FText::FromString(InUSDZFilePath)
			));
		}

		return true;
#else
		return false;
#endif	  // USE_USD_SDK
	}

	void ProcessExtraInformation(UInterchangeBaseNodeContainer& NodeContainer, UE::FUsdStage Stage)
	{
#if USE_USD_SDK
		UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&NodeContainer);

		TMap<FString, FString> MetadataMap;
		UsdUtils::ReadStageMetaData(Stage, MetadataMap);

		for (TPair<FString, FString>& MetaDataEntry : MetadataMap)
		{
			SourceNode->SetExtraInformation(MetaDataEntry.Key, MetaDataEntry.Value);
		}
#endif	  // USE_USD_SDK
	}

	void AddAnalytics(TObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler, UInterchangeUsdContext* Context)
	{
#if USE_USD_SDK
		if (!AnalyticsHandler || !Context)
		{
			return;
		}

		FInterchangeOpenUSDImportModule& OpenUSDModule = FModuleManager::LoadModuleChecked<FInterchangeOpenUSDImportModule>("InterchangeOpenUSDImport"
		);

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("NumTotalHandlers"), UE::Interchange::USD::FSchemaHandlerRegistry::RegisteredHandlerEntries.Num());
		Attributes.Emplace(
			TEXT("NumCustomHandlers"),
			UE::Interchange::USD::FSchemaHandlerRegistry::RegisteredHandlerEntries.Num() - OpenUSDModule.GetNumDefaultHandlers()
		);

		// TODO: Uncomment when the render context changes are submitted
		// if (UInterchangeUsdTranslatorSettings* UsdSettings = Cast<UInterchangeUsdTranslatorSettings>(TranslatorSettings))
		// {
		// 	Attributes.Emplace(TEXT("NumReorderedHandlers"), UsdSettings->CustomHandlerEntries.Num());

		// 	int32 NumCustomRenderContextHandlers = 0;
		// 	int32 NumDisabledHandlers = 0;
		// 	for (const FSchemaHandlerEntry& Entry : UsdSettings->CustomHandlerEntries)
		// 	{
		// 		if (!Entry.bEnabled)
		// 		{
		// 			NumDisabledHandlers += 1;
		// 		}

		// 		if (Entry.DefaultRenderContexts != Entry.CustomRenderContexts)
		// 		{
		// 			NumCustomRenderContextHandlers += 1;
		// 		}
		// 	}
		// 	Attributes.Emplace(TEXT("NumDisabledHandlers"), NumDisabledHandlers);
		// 	Attributes.Emplace(TEXT("NumCustomRenderContextHandlers"), NumCustomRenderContextHandlers);
		// }

		// Clear before adding the events because we may be doing multiple translations on the same import while tweaking the
		// translator settings or etc., and we don't want to pool duplicate attributes onto the same event
		const static FString BaseEventIdentifier = TEXT("Interchange.Usage.Import.USD");
		AnalyticsHandler->Clear(BaseEventIdentifier);
		AnalyticsHandler->Append(BaseEventIdentifier, Attributes);

		if (UE::FUsdStage Stage = Context->GetUsdStage())
		{
			TArray<FAnalyticsEventAttribute> CustomSchemaAttributes;
			UsdUtils::CollectSchemaAnalytics(Stage, CustomSchemaAttributes);

			const static FString CustomSchemaEventIdentifier = TEXT("Interchange.Usage.USD.CustomSchemaCount");
			AnalyticsHandler->Clear(CustomSchemaEventIdentifier);
			AnalyticsHandler->Append(CustomSchemaEventIdentifier, CustomSchemaAttributes);
		}
#endif	  // USE_USD_SDK
	}

	// Forwards USD log messages produced inside an FScopedUsdMessageLog into the Interchange
	// results container, tagging each one with the source layer's identifier when available.
	//
	// Lives here as a free function rather than as a member of UInterchangeUsdContext because
	// the FScopedUsdMessageLog that drives it must be stack-local within Translate() to keep
	// its underlying pxr::TfErrorMark constructed and destructed on the same thread (see the
	// matching FScopedUsdMessageLog declaration in Translate). pxr::TfErrorMark uses a
	// thread-local counter (tbb::enumerable_thread_specific in TfDiagnosticMgr); a cross-thread
	// destruction underflows the destroying thread's counter from 0 to SIZE_MAX, after which
	// USD's diagnostic state is permanently broken on that thread for the rest of the process.
	void DisplayUsdLogMessages(
		UInterchangeResultsContainer* ResultsContainer,
		const UE::FUsdStage& Stage,
		const TArray<TSharedRef<FTokenizedMessage>>& Messages
	)
	{
		if (!ResultsContainer)
		{
			return;
		}

		FString SourceAssetName;
		if (Stage)
		{
			SourceAssetName = Stage.GetRootLayer().GetIdentifier();
		}

		for (const TSharedRef<FTokenizedMessage>& Message : Messages)
		{
			switch (Message->GetSeverity())
			{
				case EMessageSeverity::Error:
				{
					UInterchangeResultError_Generic* Result = ResultsContainer->Add<UInterchangeResultError_Generic>();
					Result->Text = Message->ToText();
					Result->SourceAssetName = SourceAssetName;
					break;
				}
				case EMessageSeverity::Warning:
				case EMessageSeverity::PerformanceWarning:
				{
					UInterchangeResultWarning_Generic* Result = ResultsContainer->Add<UInterchangeResultWarning_Generic>();
					Result->Text = Message->ToText();
					Result->SourceAssetName = SourceAssetName;
					break;
				}
				case EMessageSeverity::Info:
				default:
				{
					UInterchangeResultDisplay_Generic* Result = ResultsContainer->Add<UInterchangeResultDisplay_Generic>();
					Result->Text = Message->ToText();
					Result->SourceAssetName = SourceAssetName;
					break;
				}
			}
		}
	}
}	 // namespace UE::InterchangeUsdTranslator::Private

UInterchangeUsdTranslatorSettings::UInterchangeUsdTranslatorSettings()
	: GeometryPurpose((int32)(EUsdPurpose::Default | EUsdPurpose::Proxy | EUsdPurpose::Render | EUsdPurpose::Guide))
	, RenderContext(UnrealIdentifiers::UnrealRenderContext)
	, MaterialPurpose(*UnrealIdentifiers::MaterialPreviewPurpose)
	, InterpolationType(EUsdInterpolationType::Linear)
	, bOverrideStageOptions(false)
	, StageOptions{
		  0.01,				   // MetersPerUnit
		  EUsdUpAxis::ZAxis	   // UpAxis
	  }
	, PointInstancerCollapsing(EUsdPointInstancerCollapsing::NoCollapsing)
	, bUseSchemaForCollapsing(false)
	, bUsePrimKindsForCollapsing(false)
	, KindsToCollapse(static_cast<int32>(EUsdDefaultKind::Component | EUsdDefaultKind::Subcomponent))
	, bTranslatePrimAttributes(false) // False by default as it could be expensive to traverse all attributes of all prims running the regex
	, AttributeRegexFilter(TEXT("."))
	, bTranslatePrimMetadata(false)
	, MetadataRegexFilter(TEXT("."))
{
}

UInterchangeUSDTranslator::UInterchangeUSDTranslator()
	: Impl(MakeUnique<UE::InterchangeUsdTranslator::Private::UInterchangeUSDTranslatorImpl>())
{
}

EInterchangeTranslatorType UInterchangeUSDTranslator::GetTranslatorType() const
{
	return GInterchangeEnableUSDLevelImport ? EInterchangeTranslatorType::Scenes : EInterchangeTranslatorType::Assets;
}

EInterchangeTranslatorAssetType UInterchangeUSDTranslator::GetSupportedAssetTypes() const
{
	return EInterchangeTranslatorAssetType::Materials | EInterchangeTranslatorAssetType::Meshes | EInterchangeTranslatorAssetType::Animations | EInterchangeTranslatorAssetType::Grooms;
}

TArray<FString> UInterchangeUSDTranslator::GetSupportedFormats() const
{
#if USE_USD_SDK
	TArray<FString> Extensions;
	if (GInterchangeEnableUSDImport)
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("UnrealUSDWrapper"));
		UnrealUSDWrapper::AddUsdImportFileFormatDescriptions(Extensions);
	}
	return Extensions;
#else
	return {};
#endif	  // USE_USD_SDK
}

#if USE_USD_SDK
TOptional<UE::Interchange::USD::FHandlerAccumulatedInfo> UInterchangeUSDTranslator::TranslatePrim(
	const UE::FUsdPrim& Prim,
	UE::Interchange::USD::FTraversalInfo& TraversalInfo,
	bool bAllowSceneNodeGeneration
) const
{
	using namespace UE::Interchange::USD;
	using namespace UE::InterchangeUsdTranslator::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::TranslatePrim)

	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr || !UsdContext)
	{
		return {};
	}

	const UE::FSdfPath PrimPath = Prim.GetPrimPath();

	// We already translated these prims, so let's return our cached result.
	// This is useful because we may call TranslatePrim() multiple times on e.g. a Material prim when parsing
	// material assignments from meshes, and we don't want to keep translating the same prim over and over.
	if (UE::Interchange::USD::FHandlerAccumulatedInfo* ExistingInfo = UsdContext->HandledPrimInfo.Find(PrimPath))
	{
		return *ExistingInfo;
	}

	// Here we add the prim path to HandledPrimInfo before we call OnTranslate on our handlers to prevent infinite loops,
	// in case these handlers also end up calling TranslatePrim for this very same prim, for some reason.
	//
	// Note that we take AccumulatedInfo by value here, pass it through all the handlers, and only later add it back to
	// HandledPrimInfo, overwriting the entry in the map with the updated value. We intentionally don't just keep a direct
	// reference to an AccumulatedInfo value inside the HandledPrimInfo map, because it's possible that the OnTranslate
	// calls trigger additional prim translations, that may add other entries to HandledPrimInfo. If it causes the map to
	// reallocate, it could invalidate the reference
	UE::Interchange::USD::FHandlerAccumulatedInfo AccumulatedInfo = UsdContext->HandledPrimInfo.Emplace(PrimPath);
	{
		// By temporarily pushing nullptr scene node here, any call to AccumulatedInfo.GetMainSceneNode() will return nullptr.
		// This is sort of a trick to prevent creating scene nodes internally if the caller is just interested on the produced
		// asset nodes
		bool bInsertedNullptrSceneNode = false;
		if (!bAllowSceneNodeGeneration)
		{
			if (AccumulatedInfo.PrimSceneNodes.Num() == 0 || AccumulatedInfo.PrimSceneNodes[0] != nullptr)
			{
				AccumulatedInfo.PrimSceneNodes.Insert(nullptr, 0);
				bInsertedNullptrSceneNode = true;
			}
		}

		// For now just emit a single scene node for the pseudoroot
		if (Prim.IsPseudoRoot())
		{
			AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, *UsdContext);
		}
		else
		{
			for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
			{
				if (Handler->IsEnabled() && Handler->CanHandlePrim(Prim, *UsdContext))
				{
					Handler->OnTranslate(Prim, TraversalInfo, AccumulatedInfo, *UsdContext);
				}
			}
		}

		if (bInsertedNullptrSceneNode && AccumulatedInfo.PrimSceneNodes.Num() > 0)
		{
			AccumulatedInfo.PrimSceneNodes.RemoveAt(0);
		}
	}
	UsdContext->HandledPrimInfo.Add(PrimPath, AccumulatedInfo);

	return AccumulatedInfo;
}

void UInterchangeUSDTranslator::TranslatePrimSubtree(const UE::FUsdPrim& Prim, const UE::Interchange::USD::FTraversalInfo& TraversalInfo, bool bAllowSceneNodeGeneration, UE::Interchange::USD::FHandlerAccumulatedInfo* SubTreeAccumulatedInfo) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::TranslatePrimSubtree)

	using namespace UE::Interchange::USD;
	using namespace UE::InterchangeUsdTranslator::Private;

	if (!UsdContext)
	{
		return;
	}

	const UE::FSdfPath PrimPath = Prim.GetPrimPath();
	if (UsdContext->HandledPrimInfo.Contains(PrimPath))
	{
		// If we already handled Prim, let's assume we already handled its entire subtree as well

		if (SubTreeAccumulatedInfo)
		{
			SubTreeAccumulatedInfo->AppendInfo(UsdContext->HandledPrimInfo[PrimPath]);
		}
		return;
	}

	// Honor PrimsToImport: skip subtrees that are neither at/under any allowed path nor an
	// ancestor of one (the ancestor case lets traversal walk down through e.g. /Root to reach
	// /Root/MyTarget). An empty ParsedPrimsToImport means "no filter" (default {"/"}).
	if (!UsdContext->ParsedPrimsToImport.IsEmpty())
	{
		bool bShouldVisit = false;
		for (const UE::FSdfPath& AllowedPath : UsdContext->ParsedPrimsToImport)
		{
			if (PrimPath.HasPrefix(AllowedPath) || AllowedPath.HasPrefix(PrimPath))
			{
				bShouldVisit = true;
				break;
			}
		}
		if (!bShouldVisit)
		{
			return;
		}
	}

	// Do this before generating other nodes as they may need the updated info
	FTraversalInfo TraversalInfoCopy = TraversalInfo;
	TraversalInfoCopy.UpdateWithCurrentPrim(Prim);

	TOptional<FHandlerAccumulatedInfo> AccumulatedHandlerInfo = TranslatePrim(Prim, TraversalInfoCopy, bAllowSceneNodeGeneration);
	if (AccumulatedHandlerInfo.IsSet())
	{
		TraversalInfoCopy.ParentNode = AccumulatedHandlerInfo.GetValue().GetMainSceneNode();

		if (SubTreeAccumulatedInfo)
		{
			SubTreeAccumulatedInfo->AppendInfo(AccumulatedHandlerInfo.GetValue());
		}
	}

	constexpr bool bTraverseInstanceProxies = true;
	for (const UE::FUsdPrim& ChildPrim : Prim.GetFilteredChildren(bTraverseInstanceProxies))
	{
		TranslatePrimSubtree(ChildPrim, TraversalInfoCopy, bAllowSceneNodeGeneration, SubTreeAccumulatedInfo);
	}
}

const TArray<TSharedRef<UE::Interchange::USD::FSchemaHandler>>& UInterchangeUSDTranslator::GetCurrentSchemaHandlers() const
{
	using namespace UE::InterchangeUsdTranslator::Private;

	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		const static TArray<TSharedRef<UE::Interchange::USD::FSchemaHandler>> Empty;
		return Empty;
	}

	return ImplPtr->SchemaHandlers;
}
#endif	  // USE_USD_SDK

bool UInterchangeUSDTranslator::Translate(UInterchangeBaseNodeContainer& NodeContainer) const
{
#if USE_USD_SDK
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::Translate)

	using namespace UE;
	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UsdToUnreal;

	UInterchangeUsdTranslatorSettings* Settings = Cast<UInterchangeUsdTranslatorSettings>(GetSettings());
	if (!Settings)
	{
		return false;
	}

	if (!SourceData)
	{
		return false;
	}

	// Fetch or create context
	UObject* ContextObject = SourceData->GetContextObjectByTag(UE::Interchange::USD::USDContextTag);
	UInterchangeUsdContext* Context = Cast<UInterchangeUsdContext>(ContextObject);
	if (!Context)
	{
		Context = NewObject<UInterchangeUsdContext>(const_cast<UInterchangeUSDTranslator*>(this));

		ensureMsgf(
			!ContextObject,
			TEXT("Invalid ContextObject with tag '%s' will be removed and replaced with an UInterchangeUsdContext object"),
			*UE::Interchange::USD::USDContextTag
		);
		SourceData->SetContextObjectByTag(UE::Interchange::USD::USDContextTag, Context);
	}
	if (!ensure(Context))
	{
		return false;
	}

	// We used to have the message log live inside the UsdContext, but that can be problematic
	// because the context may be initialized from here, and reset from the game thread in e.g.
	// ReleaseResources(). The message log internally uses pxr::TfErrorMark, which is a
	// *per-thread* counter however, which would cause lots of trouble. So now we use a scoped
	// message log for this function only, so that we can guarantee it's created and destroyed
	// on the same thread
	UInterchangeResultsContainer* ResultsContainer = Results;
	FScopedUsdMessageLog ScopedMessageLog(
		[ResultsContainer, Context](const TArray<TSharedRef<FTokenizedMessage>>& Messages)
		{
			DisplayUsdLogMessages(ResultsContainer, Context->GetUsdStage(), Messages);
		}
	);

	// Careful: Initialize will reset our cached USD stage as it internally calls Reset().
	// If we have been provided a stage from an external context (a context we create
	// ourselves up here won't have one) then we want to retain that stage even after we
	// "initialize"
	//
	// TODO: We need a better way of initializing this context that doesn't wipe what we want
	// to keep
	UE::FUsdStage ExistingStage = Context->GetUsdStage();
	FString USDZFilePath = Context->USDZFilePath;
	FString DecompressedUSDZRoot = Context->DecompressedUSDZRoot;
	{
		Context->Initialize(const_cast<UInterchangeUSDTranslator*>(this), &NodeContainer);
	}
	if (ExistingStage)
	{
		Context->SetUsdStage(ExistingStage);
	}
	
	UsdContext = Context;

	// Parse PrimsToImport into FSdfPaths once so TranslatePrimSubtree (and any schema handlers
	// driving traversal directly) can do cheap path-prefix comparisons. The default of {"/"} (or
	// any entry that's the absolute root) means "no filter" -- leave ParsedPrimsToImport empty
	// in that case to keep the fast path free.
	Context->ParsedPrimsToImport.Reset();
	for (const FString& PrimPathString : Settings->PrimsToImport)
	{
		if (PrimPathString.IsEmpty())
		{
			continue;
		}

		UE::FSdfPath Path{*PrimPathString};
		if (Path.IsAbsoluteRootPath())
		{
			// Any "/" entry trivially admits the entire stage -- filter is a no-op.
			Context->ParsedPrimsToImport.Reset();
			break;
		}
		Context->ParsedPrimsToImport.Add(MoveTemp(Path));
	}

	// Setup stage
	UE::FUsdStage StageToImport = Context->GetUsdStage();
	{
		// Context didn't provide a stage: Try loading one from the provided file path
		if (!StageToImport)
		{
			FString FilePath = SourceData->GetFilename();
			if (!FPaths::FileExists(FilePath))
			{
				return false;
			}

			// If we're provided a USDZ path, for now we will decompress it into a temp dir and redirect our paths.
			//
			// This is mainly because the texture factories must receive a simple file path in order to produce the
			// their payloads. It's not practical to make them handle USDZ files, and it's not yet possible
			// to provide them with raw binary buffers directly either
			if (DecompressUSDZFileToTempFolder(FilePath, DecompressedUSDZRoot))
			{
				USDZFilePath = FilePath;
				FilePath = DecompressedUSDZRoot;
			}

			// Import should always feel like it's directly from disk, so we ignore already loaded layers and stage cache
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(OpenStage)

				const bool bUseStageCache = false;
				const bool bForceReloadLayersFromDisk = true;
				StageToImport = UnrealUSDWrapper::OpenStage(*FilePath, EUsdInitialLoadSet::LoadAll, bUseStageCache, bForceReloadLayersFromDisk);
			}
		}

		if (!StageToImport)
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("FailedToOpenStage", "Failed to get a valid stage from file '{0}'. Does the file exist and is it a valid USD file?"),
				FText::FromString(SourceData->GetFilename())
			));

			return false;
		}

		if (Settings)
		{
			// Apply coordinate system conversion to the stage if we have one
			if (Settings->bOverrideStageOptions)
			{
				UsdUtils::SetUsdStageMetersPerUnit(StageToImport, Settings->StageOptions.MetersPerUnit);
				UsdUtils::SetUsdStageUpAxis(StageToImport, Settings->StageOptions.UpAxis);
			}

			StageToImport.SetInterpolationType(Settings->InterpolationType);
		}

		ProcessExtraInformation(NodeContainer, StageToImport);
	}

	// Update context with our newly opened stage
	if (!ensure(Context->SetUsdStage(StageToImport)))
	{
		return false;
	}
	Context->USDZFilePath = USDZFilePath;
	Context->DecompressedUSDZRoot = DecompressedUSDZRoot;

	Impl = MakeUnique<UE::InterchangeUsdTranslator::Private::UInterchangeUSDTranslatorImpl>();
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return false;
	}

	// Generate handlers for the translation
	const TArray<FSchemaHandlerEntry>* HandlerEntries = &FSchemaHandlerRegistry::RegisteredHandlerEntries;
	if (Settings && Settings->CustomHandlerEntries.Num() > 0)
	{
		HandlerEntries = &Settings->CustomHandlerEntries;
	}
	ImplPtr->SchemaHandlers = FSchemaHandlerRegistry::GenerateHandlers(HandlerEntries);

	// Build info cache now that everything is setup
	Context->SetupInterchangeInfoCache();

	// Traverse stage and emit translated nodes
	{
		FTraversalInfo Info;
		TranslatePrimSubtree(StageToImport.GetPseudoRoot(), Info);
	}

	AddAnalytics(AnalyticsHandler, Context);

	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

void UInterchangeUSDTranslator::ReleaseSource()
{
	Super::ReleaseSource();

#if USE_USD_SDK
	if (UsdContext)
	{
		// If we decompressed a USDZ file to a temp folder this will delete everything from that folder
		if (!UsdContext->DecompressedUSDZRoot.IsEmpty())
		{
			const bool bRequireExists = false;
			const bool bTree = true;
			IFileManager::Get().DeleteDirectory(*FPaths::GetPath(UsdContext->DecompressedUSDZRoot), bRequireExists, bTree);
		}
		UsdContext->Reset();
		UsdContext = nullptr;
	}
#endif	  // USE_USD_SDK
}

void UInterchangeUSDTranslator::ImportFinish()
{
	Super::ImportFinish();

	// These objects are only set by Translate so they are released here instead of ReleaseSource
	if (TranslatorSettings)
	{
		TranslatorSettings->ClearFlags(RF_Standalone);
		TranslatorSettings = nullptr;
	}

	Impl.Reset();
}

UInterchangeTranslatorSettings* UInterchangeUSDTranslator::GetSettings() const
{
	using namespace UE::InterchangeUsdTranslator::Private;

	if (!TranslatorSettings)
	{
		TranslatorSettings = DuplicateObject<UInterchangeUsdTranslatorSettings>(
			UInterchangeUsdTranslatorSettings::StaticClass()->GetDefaultObject<UInterchangeUsdTranslatorSettings>(),
			GetTransientPackage()
		);
		TranslatorSettings->LoadSettings();
		TranslatorSettings->ClearFlags(RF_ArchetypeObject);
		TranslatorSettings->SetFlags(RF_Standalone);
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
	}

	return TranslatorSettings;
}

void UInterchangeUSDTranslator::SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings)
{
	using namespace UE::InterchangeUsdTranslator::Private;

	if (TranslatorSettings)
	{
		TranslatorSettings->ClearFlags(RF_Standalone);
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		TranslatorSettings = nullptr;
	}

	if (const UInterchangeUsdTranslatorSettings* USDTranslatorSettings = Cast<UInterchangeUsdTranslatorSettings>(InterchangeTranslatorSettings))
	{
		TranslatorSettings = DuplicateObject<UInterchangeUsdTranslatorSettings>(USDTranslatorSettings, GetTransientPackage());
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		TranslatorSettings->SetFlags(RF_Standalone);
	}
}

TOptional<UE::Interchange::FMeshPayloadData> UInterchangeUSDTranslator::GetMeshPayloadData(
	const FInterchangeMeshPayLoadKey& PayloadKey,
	const UE::Interchange::FAttributeStorage& PayloadAttributes
) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetMeshPayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FMeshPayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetMeshPayloadData(PayloadKey, PayloadAttributes, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TOptional<UE::Interchange::FImportImage> UInterchangeUSDTranslator::GetTexturePayloadData(
	const FString& PayloadKey,
	TOptional<FString>& AlternateTexturePath
) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetTexturePayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FImportImage> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetTexturePayloadData(PayloadKey, AlternateTexturePath, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TOptional<UE::Interchange::FImportBlockedImage> UInterchangeUSDTranslator::GetBlockedTexturePayloadData(
	const FString& PayloadKey,
	TOptional<FString>& AlternateTexturePath
) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetBlockedTexturePayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FImportBlockedImage> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetBlockedTexturePayloadData(PayloadKey, AlternateTexturePath, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TArray<UE::Interchange::FAnimationPayloadData> UInterchangeUSDTranslator::GetAnimationPayloadData(
	const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries
) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetAnimationPayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TArray<UE::Interchange::FAnimationPayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetAnimationPayloadData(PayloadQueries, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TOptional<UE::Interchange::FVolumePayloadData> UInterchangeUSDTranslator::GetVolumePayloadData(const UE::Interchange::FVolumePayloadKey& PayloadKey) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetVolumePayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FVolumePayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetVolumePayloadData(PayloadKey, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TOptional<UE::Interchange::FGroomPayloadData> UInterchangeUSDTranslator::GetGroomPayloadData(const FInterchangeGroomPayloadKey& PayloadKey) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetGroomPayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FGroomPayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetGroomPayloadData(PayloadKey, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TOptional<UE::Interchange::FInterchangeAudioPayloadData> UInterchangeUSDTranslator::GetAudioPayloadData(const FString& PayloadKey) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetAudioPayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TOptional<UE::Interchange::FInterchangeAudioPayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetAudioPayloadData(PayloadKey, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

TObjectPtr<UInterchangeGenericPayloadData> UInterchangeUSDTranslator::GetGenericPayloadData(const FString& PayloadKey) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeUSDTranslator::GetGenericPayloadData)

	using namespace UE::InterchangeUsdTranslator::Private;
	using namespace UE::Interchange;

	TObjectPtr<UInterchangeGenericPayloadData> Result;

#if USE_USD_SDK
	UInterchangeUSDTranslatorImpl* ImplPtr = Impl.Get();
	if (ImplPtr && UsdContext)
	{
		for (const TSharedRef<FSchemaHandler>& Handler : ImplPtr->SchemaHandlers)
		{
			if (Handler->IsEnabled())
			{
				Handler->OnGetGenericPayloadData(PayloadKey, *UsdContext, Result);
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

#undef LOCTEXT_NAMESPACE
