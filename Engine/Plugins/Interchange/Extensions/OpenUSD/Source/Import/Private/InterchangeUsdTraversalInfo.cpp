// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "InterchangeUsdTraversalInfo.h"

#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdSkelCache.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdVariantSets.h"

#include "CoreMinimal.h"

namespace UE::Interchange::USD
{
	void FTraversalInfo::UpdateWithCurrentPrim(const UE::FUsdPrim& CurrentPrim)
	{
		if (!CurrentPrim)
		{
			return;
		}

		bVisible = bVisible && UsdUtils::HasInheritedVisibility(CurrentPrim);

		// We only want this to be true when we're traversing the exact prim that owns
		// the LOD: Once we step into any of its children it should go back to false
		bIsLODVariantContainer = CurrentPrim.GetVariantSets().HasVariantSet(LODString);

		if (CurrentPrim.IsA(TEXT("SkelRoot")))
		{
			if (!ClosestParentSkelRootPath)
			{
				// The root-most skel cache should handle any nested UsdSkel prims as well
				FurthestSkelCache = MakeShared<UE::FUsdSkelCache>();

				const bool bTraverseInstanceProxies = true;
				FurthestSkelCache->Clear();
				FurthestSkelCache->Populate(CurrentPrim, bTraverseInstanceProxies);
				FurthestParentSkelRootPath = MakeShared<FString>(CurrentPrim.GetPrimPath().GetString());
			}

			ClosestParentSkelRootPath = MakeShared<FString>(CurrentPrim.GetPrimPath().GetString());
		}

		if (ClosestParentSkelRootPath && CurrentPrim.HasAPI(TEXT("SkelBindingAPI")))
		{
			UE::FUsdStage Stage = CurrentPrim.GetStage();

			if (UE::FUsdRelationship SkelRel = CurrentPrim.GetRelationship(TEXT("skel:skeleton")))
			{
				TArray<UE::FSdfPath> Targets;
				if (SkelRel.GetTargets(Targets) && Targets.Num() > 0)
				{
					UE::FUsdPrim TargetSkeleton = Stage.GetPrimAtPath(Targets[0]);
					if (TargetSkeleton && TargetSkeleton.IsA(TEXT("Skeleton")))
					{
						BoundSkeletonPrimPath = MakeShared<FString>(TargetSkeleton.GetPrimPath().GetString());
					}
				}
			}
		}

		if (CurrentPrim.HasAPI(*UsdToUnreal::ConvertToken(UnrealIdentifiers::NaniteAssemblySkelBindingAPI)))
		{
			ClosestNaniteAssemblySkelBindingPath = MakeShared<FString>(CurrentPrim.GetPrimPath().GetString());
		}
	}

	UE::FUsdSkelSkeletonQuery FTraversalInfo::ResolveSkelQuery(const UE::FUsdStage& Stage) const
	{
		if (!Stage || !BoundSkeletonPrimPath || BoundSkeletonPrimPath->IsEmpty() || !FurthestSkelCache.IsValid())
		{
			return {};
		}

		UE::FUsdPrim SkeletonPrim = Stage.GetPrimAtPath(UE::FSdfPath{**BoundSkeletonPrimPath});
		if (!SkeletonPrim)
		{
			return {};
		}

		return FurthestSkelCache->GetSkelQuery(SkeletonPrim);
	}

	UE::FUsdPrim FTraversalInfo::ResolveClosestParentSkelRoot(const UE::FUsdStage& Stage) const
	{
		if (!Stage || !ClosestParentSkelRootPath || ClosestParentSkelRootPath->IsEmpty())
		{
			return {};
		}

		return Stage.GetPrimAtPath(UE::FSdfPath{**ClosestParentSkelRootPath});
	}

	void FTraversalInfo::RepopulateSkelCache(const UE::FUsdStage& Stage)
	{
		if (!FurthestSkelCache.IsValid() || !FurthestParentSkelRootPath.IsValid() || FurthestParentSkelRootPath->IsEmpty())
		{
			return;
		}

		UE::FUsdPrim SkelRootPrim = Stage.GetPrimAtPath(UE::FSdfPath{**FurthestParentSkelRootPath});
		if (!SkelRootPrim)
		{
			return;
		}

		const bool bTraverseInstanceProxies = true;
		FurthestSkelCache->Clear();
		ensure(FurthestSkelCache->Populate(SkelRootPrim, bTraverseInstanceProxies));
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
