// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/CameraSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeCameraNode.h"
#include "InterchangeSceneNode.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "USDIncludesEnd.h"

namespace UE::GeomCameraSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	const FString ProjectionToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->projection);
	const FString HorizontalApertureToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->horizontalAperture);
	const FString VerticalApertureToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->verticalAperture);
	const FString HorizontalApertureOffsetToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->horizontalApertureOffset);
	const FString VerticalApertureOffsetToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->verticalApertureOffset);
	const FString FocalLengthToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->focalLength);
	const FString ClippingRangeToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->clippingRange);
	const FString FstopToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->fStop);
	const FString FocusDistanceToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->focusDistance);
	const FString ExposureToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->exposure);
	const FString ExposureIsoToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->exposureIso);
	const FString ExposureTimeToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->exposureTime);
	const FString ExposureFstopToken = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->exposureFStop);

	void ConvertCameraPrim(const UE::FUsdPrim& Prim, UInterchangeBaseCameraNode* CameraNode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddCameraNode)

		// ref. UsdToUnreal::ConvertGeomCamera
		UE::FUsdStage Stage = Prim.GetStage();
		FUsdStageInfo StageInfo{Stage};

		FName Projection = UsdUtils::GetAttributeValue<FName>(Prim, ProjectionToken);
		if (UInterchangePhysicalCameraNode* PhysicalCamera = Cast<UInterchangePhysicalCameraNode>(CameraNode))
		{
			if (float SensorWidth; UsdUtils::GetAuthoredAttributeValue<float>(Prim, HorizontalApertureToken, SensorWidth))
			{
				SensorWidth = UsdToUnreal::ConvertDistance(StageInfo, SensorWidth);
				PhysicalCamera->SetCustomSensorWidth(SensorWidth);
			}

			if (float SensorHeight; UsdUtils::GetAuthoredAttributeValue<float>(Prim, VerticalApertureToken, SensorHeight))
			{
				SensorHeight = UsdToUnreal::ConvertDistance(StageInfo, SensorHeight);
				PhysicalCamera->SetCustomSensorHeight(SensorHeight);
			}

			if (float SensorHorizontalOffset; UsdUtils::GetAuthoredAttributeValue<float>(Prim, HorizontalApertureOffsetToken, SensorHorizontalOffset))
			{
				SensorHorizontalOffset = UsdToUnreal::ConvertDistance(StageInfo, SensorHorizontalOffset);
				PhysicalCamera->SetCustomSensorHorizontalOffset(SensorHorizontalOffset);
			}

			if (float SensorVerticalOffset; UsdUtils::GetAuthoredAttributeValue<float>(Prim, VerticalApertureOffsetToken, SensorVerticalOffset))
			{
				SensorVerticalOffset = UsdToUnreal::ConvertDistance(StageInfo, SensorVerticalOffset);
				PhysicalCamera->SetCustomSensorVerticalOffset(SensorVerticalOffset);
			}

			if (float FocalLength; UsdUtils::GetAuthoredAttributeValue<float>(Prim, FocalLengthToken, FocalLength))
			{
				FocalLength = UsdToUnreal::ConvertDistance(StageInfo, FocalLength);
				PhysicalCamera->SetCustomFocalLength(FocalLength);
			}

			if (FVector2f ClippingRange; UsdUtils::GetAuthoredAttributeValue<FVector2f>(Prim, ClippingRangeToken, ClippingRange))
			{
				ClippingRange.X = UsdToUnreal::ConvertDistance(StageInfo, ClippingRange.X);
				PhysicalCamera->SetCustomNearClipPlane(ClippingRange.X);
			}

			if (float Fstop; UsdUtils::GetAuthoredAttributeValue<float>(Prim, FstopToken, Fstop))
			{
				PhysicalCamera->SetCustomDepthOfFieldFstop(Fstop);

				// Follows the documentation on the fStop attribute "Defaults to 0.0, which turns off depth of field effects."
				// We check for timeSamples too however, otherwise we'd fully turn off depth of field for cameras that only had
				// animated fStop values...
				UE::FUsdAttribute FstopAttr = Prim.GetAttribute(*FstopToken);
				if (FMath::IsNearlyZero(Fstop) && !FstopAttr.ValueMightBeTimeVarying())
				{
					PhysicalCamera->SetCustomEnableDepthOfField(false);
				}
			}

			if (float FocusDistance; UsdUtils::GetAuthoredAttributeValue<float>(Prim, FocusDistanceToken, FocusDistance))
			{
				FocusDistance = UsdToUnreal::ConvertDistance(StageInfo, FocusDistance);
				PhysicalCamera->SetCustomFocusDistance(FocusDistance);
			}
		}
		else if (UInterchangeStandardCameraNode* StandardCamera = Cast<UInterchangeStandardCameraNode>(CameraNode))
		{
			StandardCamera->SetCustomProjectionMode(EInterchangeCameraProjectionType::Orthographic);

			// We're a bit more verbose here so that we only try using an aspect ratio if we know we have both
			// valid height and width values authored
			if (float UsdWidth; UsdUtils::GetAuthoredAttributeValue<float>(Prim, HorizontalApertureToken, UsdWidth))
			{
				float Width = UsdToUnreal::ConvertDistance(StageInfo, UsdWidth);
				StandardCamera->SetCustomWidth(Width);

				if (float UsdHeight; UsdUtils::GetAuthoredAttributeValue<float>(Prim, VerticalApertureToken, UsdHeight))
				{
					float Height = UsdToUnreal::ConvertDistance(StageInfo, UsdHeight);

					if (Height != 0.0f)
					{
						float AspectRatio = Width / Height;
						StandardCamera->SetCustomAspectRatio(AspectRatio);
					}
				}
			}

			if (FVector2f ClippingRange; UsdUtils::GetAuthoredAttributeValue<FVector2f>(Prim, ClippingRangeToken, ClippingRange))
			{
				ClippingRange = UsdUtils::GetAttributeValue<FVector2f>(Prim, ClippingRangeToken);
				ClippingRange.X = UsdToUnreal::ConvertDistance(StageInfo, ClippingRange.X);
				ClippingRange.Y = UsdToUnreal::ConvertDistance(StageInfo, ClippingRange.Y);
				StandardCamera->SetCustomNearClipPlane(ClippingRange.X);
				StandardCamera->SetCustomFarClipPlane(ClippingRange.Y);
			}
		}

		// Common attributes
		if (CameraNode)
		{
			if (float ExposureCompensation; UsdUtils::GetAuthoredAttributeValue<float>(Prim, ExposureToken, ExposureCompensation))
			{
				CameraNode->SetCustomExposureCompensation(ExposureCompensation);
			}

			bool bNeedsManualExposure = false;

			if (float ExposureIso; UsdUtils::GetAuthoredAttributeValue<float>(Prim, ExposureIsoToken, ExposureIso))
			{
				CameraNode->SetCustomExposureISO(ExposureIso);
				bNeedsManualExposure = true;
			}

			if (float ExposureTime; UsdUtils::GetAuthoredAttributeValue<float>(Prim, ExposureTimeToken, ExposureTime))
			{
				CameraNode->SetCustomExposureShutterSpeed(ExposureTime != 0.0f ? (1.0f / ExposureTime) : 0.0f);
				bNeedsManualExposure = true;
			}

			if (float ExposureFstop; UsdUtils::GetAuthoredAttributeValue<float>(Prim, ExposureFstopToken, ExposureFstop))
			{
				CameraNode->SetCustomExposureFstop(ExposureFstop);
				bNeedsManualExposure = true;
			}

			// We likely want manual exposure if we have any of these authored.
			// Note that ExposureCompensation itself doesn't matter though: It can still have an effect even with
			// auto exposure enabled
			if (bNeedsManualExposure)
			{
				CameraNode->SetCustomAutoExposureMethod(EInterchangeAutoExposureMethod::Manual);
			}
		}
	}
}	 // namespace UE::GeomCameraSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FCameraSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("CameraHandler");
		return HandlerName;
	}

	const FString& FCameraSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Camera");
		return SchemaName;
	}

	TOptional<bool> FCameraSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FCameraSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FCameraSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCameraSchemaHandler::OnTranslate)

		using namespace UE::GeomCameraSchemaHandler::Private;

		const FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, CameraPrefix);
		const FString NewNodeName{Prim.GetName().ToString()};

		// Let's try reusing whatever camera node was provided
		UInterchangeBaseCameraNode* AssetNode = AccumulatedInfo.GetAssetNodeOfClass<UInterchangeBaseCameraNode>();
		if (!AssetNode)
		{
			UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
			if (!NodeContainer)
			{
				return false;
			}

			// If we get to create one ourselves then let's create a standard camera for orthographic projection,
			// and use the physical camera for perspective
			FName Projection = UsdUtils::GetAttributeValue<FName>(Prim, ProjectionToken);
			if (Projection == TEXT("orthographic"))
			{
				AssetNode = NewObject<UInterchangeStandardCameraNode>(NodeContainer);
			}
			else
			{
				AssetNode = NewObject<UInterchangePhysicalCameraNode>(NodeContainer);
			}

			NodeContainer->SetupNode(AssetNode, NewNodeUid, NewNodeName, EInterchangeNodeContainerType::TranslatedAsset);
			AccumulatedInfo.PrimAssetNodes.Add(AssetNode);
		}

		ConvertCameraPrim(Prim, AssetNode);

		const static TMap<FString, TArray<FInterchangeTrackInfo>> OrthoAttributeMapping = {
			{HorizontalApertureToken, 	{{UnrealIdentifiers::OrthoWidthPropertyName, 			EInterchangePropertyTracks::CameraOrthoWidth},
									   	 {UnrealIdentifiers::AspectRatioPropertyName, 			EInterchangePropertyTracks::CameraAspectRatio}}},
			{VerticalApertureToken,   	{{UnrealIdentifiers::AspectRatioPropertyName, 			EInterchangePropertyTracks::CameraAspectRatio}}},
			{ClippingRangeToken,      	{{UnrealIdentifiers::OrthoNearClipPlanePropertyName, 	EInterchangePropertyTracks::CameraOrthoNearClipPlane},
									   	 {UnrealIdentifiers::OrthoFarClipPlanePropertyName, 	EInterchangePropertyTracks::CameraOrthoFarClipPlane}}},
			{ExposureToken,      		{{UnrealIdentifiers::ExposureCompensationPropertyName, 	EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureBias}}},
			{ExposureIsoToken,      	{{UnrealIdentifiers::CameraISOPropertyName, 			EInterchangePropertyTracks::CameraPostProcessSettingsCameraISO}}},
			{ExposureTimeToken,      	{{UnrealIdentifiers::CameraShutterSpeedPropertyName, 	EInterchangePropertyTracks::CameraPostProcessSettingsCameraShutterSpeed}}},
			{ExposureFstopToken,      	{{UnrealIdentifiers::DepthOfFieldFstopPropertyName, 	EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFstop}}},
		};

		const static TMap<FString, TArray<FInterchangeTrackInfo>> PerspectiveAttributeMapping = {
			{HorizontalApertureToken, 		{{UnrealIdentifiers::SensorWidthPropertyName, 				EInterchangePropertyTracks::CameraFilmbackSensorWidth}}},
			{VerticalApertureToken,   		{{UnrealIdentifiers::SensorHeightPropertyName, 				EInterchangePropertyTracks::CameraFilmbackSensorHeight}}},
			{HorizontalApertureOffsetToken,	{{UnrealIdentifiers::SensorHorizontalOffsetPropertyName, 	EInterchangePropertyTracks::CameraFilmbackSensorHorizontalOffset}}},
			{VerticalApertureOffsetToken, 	{{UnrealIdentifiers::SensorVerticalOffsetPropertyName, 		EInterchangePropertyTracks::CameraFilmbackSensorVerticalOffset}}},
			{FocalLengthToken, 				{{UnrealIdentifiers::CurrentFocalLengthPropertyName, 		EInterchangePropertyTracks::CameraCurrentFocalLength}}},
			{ClippingRangeToken,      		{{UnrealIdentifiers::CustomNearClipppingPlanePropertyName, 	EInterchangePropertyTracks::CameraCustomNearClippingPlane}}},
			{FstopToken,      				{{UnrealIdentifiers::DepthOfFieldFstopPropertyName, 		EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFstop}}},
			{FocusDistanceToken,      		{{UnrealIdentifiers::ManualFocusDistancePropertyName, 		EInterchangePropertyTracks::CameraFocusSettingsManualFocusDistance}}},
			{ExposureToken,      			{{UnrealIdentifiers::ExposureCompensationPropertyName, 		EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureBias}}},
			{ExposureIsoToken,      		{{UnrealIdentifiers::CameraISOPropertyName, 				EInterchangePropertyTracks::CameraPostProcessSettingsCameraISO}}},
			{ExposureTimeToken,      		{{UnrealIdentifiers::CameraShutterSpeedPropertyName, 		EInterchangePropertyTracks::CameraPostProcessSettingsCameraShutterSpeed}}},
			{ExposureFstopToken,      		{{UnrealIdentifiers::CurrentAperturePropertyName, 			EInterchangePropertyTracks::CameraCurrentAperture}}},
		};

		const bool bNodeIsOrtho = AssetNode->IsA<UInterchangeStandardCameraNode>();
		AddNodesForAnimatedAttributes(
			Prim,
			bNodeIsOrtho ? OrthoAttributeMapping : PerspectiveAttributeMapping, 
			AccumulatedInfo, 
			UsdContext
		);

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			SceneNode->SetCustomAssetInstanceUid(AssetNode->GetUniqueID());
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
