// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/CollapsingSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDErrorUtils.h"
#include "USDPrimConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdTyped.h"
#include "UsdWrappers/UsdVariantSets.h"

#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "CollapsingSchemaHandler"

namespace UE::CollapsingSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	void RecursiveCollapseSceneHierarchy(
		const UE::FUsdPrim& Prim,
		FTraversalInfo TraversalInfo,
		TArray<FTransform> TransformsToMeshBundleRoot,
		TMap<FString, TArray<FTransform>>& CurrentMeshToTransforms,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	);

	void RecursiveCollapsePointInstancer(
		const UE::FUsdPrim& PointInstancerPrim,
		FTraversalInfo TraversalInfo,
		const TArray<FTransform>& TransformsToMeshBundleRoot,
		TMap<FString, TArray<FTransform>>& CurrentMeshToTransforms,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveCollapsePointInstancer)

		UsdUtils::FUsdPointInstancerInstancesData InstancesData;
		UsdUtils::GetPointInstancerInstancesData(PointInstancerPrim, InstancesData);

		UE::FUsdStage Stage = PointInstancerPrim.GetStage();

		for (const UsdUtils::FUsdPointInstancerInstancesData::FUsdPrototypeInstances& Prototype : InstancesData.Prototypes)
		{
			const FString& PrototypePath = Prototype.PathString;
			const TArray<FTransform>& PrototypeTransforms = Prototype.InstanceTransforms;

			if (UE::FUsdPrim PrototypePrim = Stage.GetPrimAtPath(UE::FSdfPath{*PrototypePath}))
			{
				// TransformsToMeshBundleRoot represents how many times this point instancer has itself been instanced,
				// so here we have to have to produce a sort of cartesian product, so that for every instance of the
				// point instancer, we still track each of its prototype transforms
				TArray<FTransform> CartesianProductTransforms;
				CartesianProductTransforms.Reserve(TransformsToMeshBundleRoot.Num() * PrototypeTransforms.Num());
				for (const FTransform& PointInstancerToMeshBundleRoot : TransformsToMeshBundleRoot)
				{
					for (const FTransform& InstanceToPointInstancer : PrototypeTransforms)
					{
						CartesianProductTransforms.Add(InstanceToPointInstancer * PointInstancerToMeshBundleRoot);
					}
				}

				RecursiveCollapseSceneHierarchy(
					PrototypePrim,
					TraversalInfo,
					CartesianProductTransforms,
					CurrentMeshToTransforms,
					AccumulatedInfo,
					UsdContext
				);
			}
		}
	}

	void RecursiveCollapseSceneHierarchy(
		const UE::FUsdPrim& Prim,
		FTraversalInfo TraversalInfo,
		TArray<FTransform> TransformsToMeshBundleRoot, 	// We may have multiple transforms even for simple Xform prims if we're getting here from a point instancer hierarchy
		TMap<FString, TArray<FTransform>>& CurrentMeshToTransforms,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveCollapseSceneHierarchy)

		// Reference: RecursivelyCollapseChildMeshes() in USDGeomMeshConversion.cpp

		TraversalInfo.UpdateWithCurrentPrim(Prim);

		// If a prim is invisible its entire subtree must be ignored
		if (!TraversalInfo.bVisible)
		{
			return;
		}

		// Update our transform to root if we can
		FTransform PrimToParentTransform = FTransform::Identity;
		if (Prim.IsA(TEXT("Xformable")))
		{
			bool bResetTransformStack = false;
			UsdToUnreal::ConvertXformable(
				Prim.GetStage(),
				UE::FUsdTyped(Prim),
				PrimToParentTransform,
				UsdUtils::GetEarliestTimeCode(),
				&bResetTransformStack
			);

			if (!PrimToParentTransform.Equals(FTransform::Identity))
			{
				for (FTransform& TransformToMeshBundleRoot : TransformsToMeshBundleRoot)
				{
					TransformToMeshBundleRoot = PrimToParentTransform * TransformToMeshBundleRoot;
				}
			}
		}

		// Do our own thing for PointInstancers because the regular schema handler will stash the prototype
		// instances in scene component nodes, and we won't be spawning any in here
		if (Prim.IsA(TEXT("PointInstancer")))
		{
			RecursiveCollapsePointInstancer(Prim, TraversalInfo, TransformsToMeshBundleRoot, CurrentMeshToTransforms, AccumulatedInfo, UsdContext);
		}
		else
		{
			// Translate and try producing some mesh nodes for this prim
			if (UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator())
			{
				const bool bAllowSceneNodeGeneration = false;
				TOptional<FHandlerAccumulatedInfo> PrimAccumulatedInfo = CurrentTranslator->TranslatePrim(Prim, TraversalInfo, bAllowSceneNodeGeneration);

				if (PrimAccumulatedInfo)
				{
					AccumulatedInfo.AppendInfo(PrimAccumulatedInfo.GetValue());

					for (UInterchangeBaseNode* AssetNode : PrimAccumulatedInfo->PrimAssetNodes)
					{
						if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(AssetNode))
						{
							// Skip morph target nodes: they're linked to their parent mesh via
							// SetMorphTargetDependencyUid and should not be part of the bundle's
							// mesh-to-transforms map
							if (MeshNode->IsMorphTarget())
							{
								continue;
							}

							// For skinned meshes, use identity transforms: their vertices are already in bind-pose
							// space relative to the skeleton, so baking the mesh-to-collapsing-root transform into
							// the vertices would incorrectly deform them.
							if (MeshNode->IsSkinnedMesh())
							{
								CurrentMeshToTransforms.FindOrAdd(MeshNode->GetUniqueID()).Add(FTransform::Identity);
							}
							else
							{
								CurrentMeshToTransforms.FindOrAdd(MeshNode->GetUniqueID()).Append(TransformsToMeshBundleRoot);
							}
						}
					}
				}
			}

			constexpr bool bTraverseInstanceProxies = true;
			for (const FUsdPrim& ChildPrim : Prim.GetFilteredChildren(bTraverseInstanceProxies))
			{
				RecursiveCollapseSceneHierarchy(
					ChildPrim,
					TraversalInfo,
					TransformsToMeshBundleRoot,
					CurrentMeshToTransforms,
					AccumulatedInfo,
					UsdContext
				);
			}
		}
	}
}

namespace UE::Interchange::USD
{
	const FString& FCollapsingSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("CollapsingHandler");
		return HandlerName;
	}

	const FString& FCollapsingSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Imageable");
		return SchemaName;
	}

	bool FCollapsingSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		UInterchangeUsdTranslatorSettings* TranslatorSettings = UsdContext.GetTranslatorSettings();
		if (!TranslatorSettings || (!TranslatorSettings->bUsePrimKindsForCollapsing && !TranslatorSettings->bUseSchemaForCollapsing))
		{
			return false;
		}

		return FSchemaHandler::CanHandlePrim(Prim, UsdContext);
	}

	bool FCollapsingSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCollapsingSchemaHandler::OnTranslate)

		using namespace UE::CollapsingSchemaHandler::Private;

		// If we're already collapsing a mesh we don't want to create a nested mesh bundle and want instead to treat this as
		// a regular prim
		if (TraversalInfo.bCurrentlyCollapsing)
		{
			return false;
		}

		// We early out for LOD variants (for all Prims part of a LOD variant/LOD Group), 
		// otherwise we would query the info cache for inactive LOD Variant prims, which would trigger an ensure in DoesPathCollapseChildren.
		if (TraversalInfo.bIsLODVariantContainer || TraversalInfo.bInsideLODVariant)
		{
			return false;
		}

		// We can early out if our prim doesn't collapse anyway
		if (FInterchangeUsdInfoCache* InfoCache = UsdContext.GetInterchangeInfoCache())
		{
			if (!InfoCache->DoesPathCollapseChildren(Prim.GetPrimPath()))
			{
				return false;
			}
		}

		// Try collapsing the hierarchy to see if it produces any collapsed mesh nodes
		TMap<FString, TArray<FTransform>> CurrentMeshToTransforms;
		TArray<FUsdPrim> Children;
		{
			TGuardValue<bool> CollapsingGuard{TraversalInfo.bCurrentlyCollapsing, true};

			TArray<FTransform> TransformsToMeshBundleRoot{FTransform::Identity};

			constexpr bool bTraverseInstanceProxies = true;
			Children = Prim.GetFilteredChildren(bTraverseInstanceProxies);

			if (Prim.IsA(TEXT("PointInstancer")))
			{
				RecursiveCollapsePointInstancer(
					Prim,
					TraversalInfo,
					TransformsToMeshBundleRoot,
					CurrentMeshToTransforms,
					AccumulatedInfo,
					UsdContext
				);
			}
			else
			{
				for (const FUsdPrim& ChildPrim : Children)
				{
					RecursiveCollapseSceneHierarchy(
						ChildPrim,
						TraversalInfo,
						TransformsToMeshBundleRoot,
						CurrentMeshToTransforms,
						AccumulatedInfo,
						UsdContext
					);
				}
			}
		}

		// Only emit a mesh bundle node if we have at least one mesh node in there.
		//
		// That should be the correct thing to do in principle, but even in practice we should do this
		// because if we end up creating a factory node from an empty mesh bundle we could run into trouble
		// when we later have no actual payload data to add to the produced asset, and it's left in some
		// incomplete state.
		bool bAtLeastOneMeshInstance = false;
		for (const TPair<FString, TArray<FTransform>>& MeshTransforms : CurrentMeshToTransforms)
		{
			if (MeshTransforms.Value.Num() > 0)
			{
				bAtLeastOneMeshInstance = true;
				break;
			}
		}
		if (!bAtLeastOneMeshInstance)
		{
			return false;
		}

		// OK, we're collapsing this subtree. Make sure all of our child prims are marked as visited, even if our recursive functions
		// potentially ignored them, as otherwise the regular translation process will continue stepping into them. Maybe in the
		// future this could be useful, so that we could "partially collapse" a hierarchy. For now it's simpler to just avoid
		// this case however
		for (const FUsdPrim& ChildPrim : Children)
		{
			UsdContext.HandledPrimInfo.FindOrAdd(ChildPrim.GetPrimPath());
		}

		// Don't use the prototype prim for the name though, as we don't want our asset named __Prototype_N
		// This is only relevant for collapsed meshes and not other asset types because this is the only case where the
		// asset gets named after the topmost prim (i.e. the instance/prototype prim itself)
		FString NewNodeName{Prim.GetName().ToString()};
		FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);

		UInterchangeMeshBundleNode* Bundle = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeMeshBundleNode>(
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);

		if (Bundle)
		{
			UE::Interchange::USD::SetPrimPath(*Bundle, Prim.GetPrimPath().GetString());
			Bundle->SetAllMeshNodesAndTransforms(CurrentMeshToTransforms);

			// Check if this bundle contains skinned meshes. If so, mark the bundle with the skeleton dependency so
			// that the pipeline knows to create a skeletal mesh instead of a static mesh.
			{
				UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
				FString BundleSkeletonDependencyUid;
				bool bHasSkinnedMesh = false;
				bool bHasStaticMesh = false;

				for (const TPair<FString, TArray<FTransform>>& MeshTransforms : CurrentMeshToTransforms)
				{
					if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(NodeContainer->GetNode(MeshTransforms.Key)))
					{
						// Morph targets should already be filtered out above, but skip defensively
						if (MeshNode->IsMorphTarget())
						{
							ensure(false);
							continue;
						}

						if (MeshNode->IsSkinnedMesh())
						{
							bHasSkinnedMesh = true;

							FString MeshSkeletonUid;
							MeshNode->GetSkeletonDependency(0, MeshSkeletonUid);

							if (BundleSkeletonDependencyUid.IsEmpty())
							{
								BundleSkeletonDependencyUid = MeshSkeletonUid;
							}
							else if (!BundleSkeletonDependencyUid.Equals(MeshSkeletonUid))
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT("SkinnedMeshMultipleSkeletons",
										"Collapsing group '{0}' contains skinned meshes bound to different skeletons. "
										"Skinned mesh collapsing requires all meshes to share the same skeleton. Skipping skinned bundle."),
									FText::FromString(Prim.GetPrimPath().GetString())
								));
								BundleSkeletonDependencyUid.Empty();
								bHasSkinnedMesh = false;
								break;
							}
						}
						else
						{
							bHasStaticMesh = true;
						}
					}
				}

				// Only mark as skinned if ALL meshes in the bundle are skinned
				if (bHasSkinnedMesh && !bHasStaticMesh && !BundleSkeletonDependencyUid.IsEmpty())
				{
					Bundle->SetCustomSkeletonDependencyUid(BundleSkeletonDependencyUid);
				}
			}

			// Only ever setup our scene node if we can collapse stuff. Otherwise we can just yield the creation of the scene
			// node to the other schema handlers
			UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
			if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
			{
				SceneNode->SetCustomAssetInstanceUid(Bundle->GetUniqueID());
			}
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
