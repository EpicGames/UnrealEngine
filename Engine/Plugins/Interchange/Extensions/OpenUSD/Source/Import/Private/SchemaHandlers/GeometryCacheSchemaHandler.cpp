// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/GeometryCacheSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDErrorUtils.h"
#include "USDPrimConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeSceneNode.h"

#define LOCTEXT_NAMESPACE "GeometryCacheSchemaHandler"

namespace UE::GeometryCacheSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	void ConvertGeometryCacheNode(
		const UE::FUsdPrim& Prim,
		UInterchangeGeometryCacheNode* GeometryCacheNode,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertGeometryCacheNode)

		const UE::FUsdStage Stage = Prim.GetStage();
		const FString PrimPath = Prim.GetPrimPath().GetString();

		GeometryCacheNode->SetPayLoadKey(PrimPath, EInterchangeMeshPayLoadType::ANIMATED);

		int32 StartFrame = FMath::FloorToInt(Stage.GetStartTimeCode());
		int32 EndFrame = FMath::CeilToInt(Stage.GetEndTimeCode());
		UsdUtils::GetAnimatedMeshTimeCodes(Stage, PrimPath, StartFrame, EndFrame);

		double TimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
		if (TimeCodesPerSecond <= 0)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"InvalidTimeCodesPerSecond",
					"Stage '{0}' has TimeCodesPerSecond set to '{1}' which is not supported for GeometryCaches, which need values greater than zero. The GeometryCache assets will be parsed as if TimeCodesPerSecond was set to 1.0"
				),
				FText::FromString(Stage.GetRootLayer().GetIdentifier()),
				TimeCodesPerSecond
			));
			TimeCodesPerSecond = 1;
		}

		// The GeometryCache module expects the end frame to be one past the last animation frame
		EndFrame += 1;

		GeometryCacheNode->SetCustomStartFrame(StartFrame);
		GeometryCacheNode->SetCustomEndFrame(EndFrame);
		GeometryCacheNode->SetCustomFrameRate(TimeCodesPerSecond);

		const bool bConstantTopology = UsdUtils::GetMeshTopologyVariance(Prim) != UsdUtils::EMeshTopologyVariance::Heterogenous;
		GeometryCacheNode->SetCustomHasConstantTopology(bConstantTopology);

		// Note: Material assignments and some other attributes are handled by the GeomGprimSchemaHandler
	}

	bool GetTransformAnimationPayloadData(
		const FString& PayloadKey,
		const UInterchangeUsdContext& UsdContext,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetTransformAnimationPayloadData)

		FString PrimPath;
		FString UEPropertyNameStr;
		bool bSplit = PayloadKey.Split(TEXT("\\"), &PrimPath, &UEPropertyNameStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!bSplit)
		{
			return false;
		}

		UE::FUsdPrim Prim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*PrimPath});
		FName UEPropertyName = *UEPropertyNameStr;
		if (!Prim || UEPropertyName == NAME_None)
		{
			return false;
		}

		TArray<double> TimeSampleUnion;

		TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, UEPropertyName);
		bool bSuccess = UE::FUsdAttribute::GetUnionedTimeSamples(Attrs, TimeSampleUnion);
		if (!bSuccess)
		{
			return false;
		}

		// Use the first timecode as an offset when indexing
		if (!TimeSampleUnion.IsEmpty())
		{
			OutPayloadData.RangeStartTime = TimeSampleUnion[0];
		}

		const bool bIgnorePrimLocalTransform = false;
		UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(Prim, UEPropertyName, bIgnorePrimLocalTransform);
		if (Reader.TransformReader)
		{
			return ReadRawTransforms(UsdContext.GetUsdStage(), TimeSampleUnion, Reader.TransformReader, OutPayloadData);
		}

		return false;
	}
}	 // namespace UE::GeometryCacheSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FGeometryCacheSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("GeometryCacheHandler");
		return HandlerName;
	}

	const FString& FGeometryCacheSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Mesh");
		return SchemaName;
	}

	bool FGeometryCacheSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSchemaHandler::CanHandlePrim)

		return UsdUtils::IsAnimatedMesh(Prim);
	}

	TOptional<bool> FGeometryCacheSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// For now let's never collapse any animated mesh
		return false;
	}

	TOptional<bool> FGeometryCacheSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// For now let's never collapse any animated mesh
		return false;
	}

	bool FGeometryCacheSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSchemaHandler::OnTranslate)

		using namespace UE::GeometryCacheSchemaHandler::Private;

		// Don't process mesh prims with disabled purposes
		if (UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings())
		{
			if (!EnumHasAllFlags(static_cast<EUsdPurpose>(Settings->GeometryPurpose), IUsdPrim::GetPurpose(Prim)))
			{
				return false;
			}
		}

		const FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);
		const FString NewNodeName{Prim.GetName().ToString()};
		UInterchangeGeometryCacheNode* GeometryCacheNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeGeometryCacheNode>(
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);
		if (!GeometryCacheNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*GeometryCacheNode, Prim.GetPrimPath().GetString());

		ConvertGeometryCacheNode(Prim, GeometryCacheNode, AccumulatedInfo, UsdContext);

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			SceneNode->SetCustomAssetInstanceUid(GeometryCacheNode->GetUniqueID());
		}

		return true;
	}

	bool FGeometryCacheSchemaHandler::OnGetMeshPayloadData(
		const FInterchangeMeshPayLoadKey& PayloadKey,
		const UE::Interchange::FAttributeStorage& PayloadAttributes,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FMeshPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSchemaHandler::OnGetMeshPayloadData)

		bool bSuccess = false;

		UsdToUnreal::FUsdMeshConversionOptions OptionsCopy = UsdContext.CachedMeshConversionOptions;
		OptionsCopy.bMergeIdenticalMaterialSlots = false;	 // This must always be false, because we need the material assignments we read from the
															 // meshes to match up with whatever we cached from AddMeshNode, in order to fixup LOD
															 // material slots

		PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::SubdivisionLevelAttributeKey}, OptionsCopy.SubdivisionLevel);
		PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Import}, reinterpret_cast<int32&>(OptionsCopy.ImportPrimvars));

		if (EInterchangeUsdPrimvar(OptionsCopy.ImportPrimvars) != EInterchangeUsdPrimvar::Standard)
		{
			if (int32 PrimvarNumber; PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Number}, PrimvarNumber)
									 == EAttributeStorageResult::Operation_Success)
			{
				OptionsCopy.PrimvarNames.Reserve(PrimvarNumber);
				for (int32 Index = 0; Index < PrimvarNumber; ++Index)
				{
					if (FString PrimvarName;
						PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Name + FString::FromInt(Index)}, PrimvarName)
						== EAttributeStorageResult::Operation_Success)
					{
						OptionsCopy.PrimvarNames.Emplace(MoveTemp(PrimvarName));
					}
				}
			}
		}

		UE::Interchange::FMeshPayloadData MeshPayloadData;
		switch (PayloadKey.Type)
		{
			case EInterchangeMeshPayLoadType::ANIMATED:
			{
				PayloadAttributes.GetAttribute(
					UE::Interchange::FAttributeKey{MeshPayload::Attributes::MeshGlobalTransform},
					OptionsCopy.AdditionalTransform
				);

				OptionsCopy.TimeCode = PayloadKey.FrameNumber;

				bSuccess = GetStaticMeshPayloadData(
					PayloadKey.UniqueId,
					UsdContext,
					OptionsCopy,
					MeshPayloadData.MeshDescription,
					MeshPayloadData.NaniteAssemblyDescription
				);
				break;
			}
			case EInterchangeMeshPayLoadType::NONE:	   // Fallthrough
			default:
				break;
		}

		if (bSuccess)
		{
			InOutPayloadData.Emplace(MeshPayloadData);
		}

		return bSuccess;
	}

	bool FGeometryCacheSchemaHandler::OnGetAnimationPayloadData(
		const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSchemaHandler::OnGetAnimationPayloadData)

		using namespace UE::GeometryCacheSchemaHandler::Private;

		TArray<int32> TransformQueryIndexes;
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadQueries.Num(); ++PayloadIndex)
		{
			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];

			if (PayloadQuery.PayloadKey.Type == EInterchangeAnimationPayLoadType::GEOMETRY_CACHE_TRANSFORM)
			{
				TransformQueryIndexes.Add(PayloadIndex);
			}
		}
		if (TransformQueryIndexes.Num() == 0)
		{
			return true;
		}

		// Import raw transforms payloads
		if (int32 TransformPayloadCount = TransformQueryIndexes.Num(); TransformPayloadCount > 0)
		{
			TArray<TArray<UE::Interchange::FAnimationPayloadData>> TransformAnimationPayloads;

			TransformAnimationPayloads.Reserve(TransformPayloadCount);
			for (int32 QueryIndex : TransformQueryIndexes)
			{
				const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[QueryIndex];
				FAnimationPayloadData AnimationPayLoadData{PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey};
				if (GetTransformAnimationPayloadData(PayloadQuery.PayloadKey.UniqueId, UsdContext, AnimationPayLoadData))
				{
					TransformAnimationPayloads.Emplace(TArray<UE::Interchange::FAnimationPayloadData>{AnimationPayLoadData});
				}
			}

			// Append the transforms results
			for (TArray<UE::Interchange::FAnimationPayloadData>& AnimationPayload : TransformAnimationPayloads)
			{
				InOutPayloadData.Append(AnimationPayload);
			}
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
