// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/GprimSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDNaniteAssemblyUtils.h"
#include "USDSkeletalDataConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelBlendShape.h"
#include "UsdWrappers/UsdSkelBlendShapeQuery.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"
#include "UsdWrappers/UsdSkelRoot.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeResult.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSceneNode.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Rendering/SkeletalMeshLODImporterData.h"
#include "StaticMeshAttributes.h"

#define LOCTEXT_NAMESPACE "GprimSchemaHandler"

namespace UE::GeomGprimSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	EInterchangeMeshCollision ConvertApproximationType(EUsdCollisionType Approximation)
	{
		// References:
		// - InterchangeGenericStaticMeshPipeline.cpp, GetCollisionMeshType()
		// - InterchangeGenericStaticMeshPipeline.cpp, AddLodDataToStaticMesh()

		switch (Approximation)
		{
			// EInterchangeMeshCollision::None means no collision, so treat EUsdCollisionType::None
			// as convex collision instead
			case EUsdCollisionType::None:
			case EUsdCollisionType::ConvexDecomposition:
			case EUsdCollisionType::ConvexHull:
			case EUsdCollisionType::MeshSimplification:
			case EUsdCollisionType::CustomMesh:
			{
				return EInterchangeMeshCollision::Convex18DOP;
			}
			case EUsdCollisionType::Sphere:
			{
				return EInterchangeMeshCollision::Sphere;
			}
			case EUsdCollisionType::Cube:
			{
				return EInterchangeMeshCollision::Box;
			}
			case EUsdCollisionType::Capsule:
			{
				return EInterchangeMeshCollision::Capsule;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		return EInterchangeMeshCollision::None;
	}

	void CreateMorphTargetNodes(
		const UE::FUsdPrim& MeshPrim,
		UInterchangeMeshNode& MeshNode,
		const FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		UE::FUsdSkelBlendShapeQuery Query{MeshPrim};
		if (!Query)
		{
			return;
		}

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		const FString MeshPrimPath = MeshPrim.GetPrimPath().GetString();

		TFunction<void(const FString&, int32, const FString&)> AddMorphTargetNode =
			[
				&MeshNode, 
				&MeshPrim, 
				&MeshPrimPath, 
				&NodeContainer,
				&UsdContext,
				&TraversalInfo, 
				&AccumulatedInfo
			]
			(const FString& MorphTargetName, int32 BlendShapeIndex, const FString& InbetweenName)
		{
			// Note: We identify a blend shape by its Mesh prim path and the blend shape index, even though
			// the blend shape itself is a full standalone prim. This is for two reasons:
			//  - We need to also read the Mesh prim's mesh data when emitting the payload, so having the Mesh path on the payload key is handy;
			//  - It could be possible for different meshes to share the same BlendShape (possibly?), so we really want a separate version of
			//    a blend shape for each mesh that uses it.
			//
			// Despite of that though, we won't use the blendshape's full path as the morph target name, so that users can get different
			// blendshapes across the model to combine into a single morph target. Interchange has an import option to let you control
			// whether they become separate morph targets or not anyway ("Merge Morph Targets with Same Name")
			const FString NodeUid = GetMorphTargetMeshNodeUid(UsdContext, MeshPrim, BlendShapeIndex, InbetweenName);
			const FString PayloadKey = GetMorphTargetMeshPayloadKey(TraversalInfo.bInsideLODVariant, MeshPrimPath, BlendShapeIndex, InbetweenName);

			UInterchangeMeshNode* MorphTargetMeshNode = GetExistingNode<UInterchangeMeshNode>(*NodeContainer, NodeUid);
			if (!MorphTargetMeshNode)
			{
				MorphTargetMeshNode = NewObject<UInterchangeMeshNode>(NodeContainer);
				NodeContainer->SetupNode(MorphTargetMeshNode, NodeUid, MorphTargetName, EInterchangeNodeContainerType::TranslatedAsset);
			}

			MorphTargetMeshNode->SetPayLoadKey(PayloadKey, EInterchangeMeshPayLoadType::MORPHTARGET);
			MorphTargetMeshNode->SetMorphTarget(true);
			MorphTargetMeshNode->SetMorphTargetName(MorphTargetName);
			MeshNode.SetMorphTargetDependencyUid(NodeUid);

			AccumulatedInfo.PrimAssetNodes.Add(MorphTargetMeshNode);
		};

		for (size_t Index = 0; Index < Query.GetNumBlendShapes(); ++Index)
		{
			UE::FUsdSkelBlendShape BlendShape = Query.GetBlendShape(Index);
			if (!BlendShape)
			{
				continue;
			}
			UE::FUsdPrim BlendShapePrim = BlendShape.GetPrim();
			const FString BlendShapeName = BlendShapePrim.GetName().ToString();

			const FString UnusedInbetweenName;
			AddMorphTargetNode(BlendShapeName, Index, UnusedInbetweenName);

			for (const UE::FUsdSkelInbetweenShape& Inbetween : BlendShape.GetInbetweens())
			{
				const FString InbetweenName = Inbetween.GetAttr().GetName().ToString();
				const FString MorphTargetName = BlendShapeName + TEXT("_") + InbetweenName;
				AddMorphTargetNode(MorphTargetName, Index, InbetweenName);
			}
		}
	}

	void ConvertMeshNode(
		const UE::FUsdPrim& MeshPrim,
		bool bIsSkinnedMesh,
		bool bIsPrimitiveShape,
		bool bIsAnimated,
		UInterchangeMeshNode* MeshNode,
		const FTraversalInfo& Info,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMeshNode)

		if (!MeshNode)
		{
			return;
		}

		const FString PrimPath = MeshPrim.GetPrimPath().GetString();
		const FString NodeName{MeshPrim.GetName().ToString()};

		FString PayloadKey = PrimPath;
		if (Info.bInsideLODVariant)
		{
			PayloadKey = LODPrefix + PayloadKey;
		}
		if (bIsPrimitiveShape)
		{
			// We are currently not supporting Skinned Primitive Shapes.
			// In theory a Skinned Mesh needs Joint influences and weights information provided,
			// However, there does not seem to be a ruleset against a Primitive Shape having SkelBindingAPI set.
			// Which means, that on a theoretical level there could be such a scenario.
			ensureMsgf(!bIsSkinnedMesh, TEXT("Unexpected scenario: Primitive Shape is skinned."));

			// For Primitive Shapes we add PrimitiveShapePrefix for the PayLoadKey,
			// in order to be able to identify the PrimitiveShape in the PayloadData acquisition phase,
			// as the Primitive Shapes require a different MeshDescription acquisition path, compared to Static Meshes.
			PayloadKey = PrimitiveShapePrefix + PayloadKey;
		}

		if (bIsSkinnedMesh && !bIsPrimitiveShape)
		{
			MeshNode->SetSkinnedMesh(true);
			MeshNode->SetPayLoadKey(PayloadKey, EInterchangeMeshPayLoadType::SKELETAL);
			if (Info.BoundSkeletonPrimPath && !Info.BoundSkeletonPrimPath->IsEmpty() && MeshNode->GetSkeletonDependeciesCount() == 0)
			{
				MeshNode->SetSkeletonDependencyUid(MakeRootBoneNodeUid(*Info.BoundSkeletonPrimPath));
			}

			CreateMorphTargetNodes(MeshPrim, *MeshNode, Info, AccumulatedInfo, UsdContext);

			// When returning the payload data later, we'll need at the very least our SkeletonQuery,
			// and possibly Nanite assembly info, so here we store the Info object into the Impl
			{
				FWriteScopeLock ScopedInfoWriteLock{UsdContext.NodeUidToCachedTraversalInfoLock};
				UsdContext.NodeUidToCachedTraversalInfo.Add(MeshNode->GetUniqueID(), Info);
			}
		}
		// Don't setup as a static mesh if we're animated, and expect a geometry cache handler to deal with this
		else if (!bIsAnimated)
		{
			MeshNode->SetPayLoadKey(PayloadKey, EInterchangeMeshPayLoadType::STATIC);

			if (UsdUtils::IsCollisionEnabledForPrim(MeshPrim))
			{
				// If the mesh prim is flagged for collision schemas AND also setup to be an FBX-style collision mesh, prefer
				// the FBX style. We do this here because if we setup both styles at the same time, then GetCollisionMeshType() from
				// InterchangeGenericStaticMeshPipeline.cpp would prefer the explicit collisions described on the translated node.
				// That seems like the right thing in general, but is not what we want for USD due to compatibility with the legacy
				// USD Importer code
				bool bSetCustomCollisionType = true;
				{
					// Check if this mesh is an FBX-style collider
					if (UE::Interchange::Private::MeshHelper::GetMeshCollisionFromName(NodeName) != EInterchangeMeshCollision::None)
					{
						bSetCustomCollisionType = false;
					}

					// Check if we have any siblings that are also FBX-style colliders pointing at this mesh prim.
					// In that case we want to disable the collider for this mesh prim itself, so that it matches legacy USD behavior
					if (bSetCustomCollisionType)
					{
						TArray<UE::FUsdPrim> Siblings = MeshPrim.GetParent().GetChildren();
						for (const UE::FUsdPrim& Sibling : Siblings)
						{
							if (Sibling == MeshPrim || !UsdUtils::IsCollisionMesh(Sibling))
							{
								continue;
							}

							FString SiblingName = Sibling.GetName().ToString();

							int32 FirstUnderscoreIndex = INDEX_NONE;
							if (SiblingName.FindChar(TEXT('_'), FirstUnderscoreIndex) && FirstUnderscoreIndex != INDEX_NONE)
							{
								FString PotentialNodeName = SiblingName.RightChop(FirstUnderscoreIndex + 1);
								if (PotentialNodeName.StartsWith(NodeName))
								{
									bSetCustomCollisionType = false;
									break;
								}
							}
						}
					}
				}

				if (bSetCustomCollisionType)
				{
					EUsdCollisionType Approximation = UsdUtils::GetCollisionApproximationType(MeshPrim);
					EInterchangeMeshCollision InterchangeApproximation = ConvertApproximationType(Approximation);
					if (InterchangeApproximation != EInterchangeMeshCollision::None)
					{
						MeshNode->SetCustomCollisionType(InterchangeApproximation);
					}
				}
			}
		}

		// Material assignments
		{
			// Note that here we only care about finding the materials that are bound for our slot dependencies, and the render context
			// we provide to this function doesn't affect that all: We'll find the the same bound material prim regardless of which we use.
			// By passing the universal render context here though, we ensure we *check* for UnrealMaterial prims, but don't actually "resolve"
			// them to the content path just yet (which is what would happen if we provided the "unreal" render context): We want to just
			// be pointing at the actual Material prims here, so that our SetSlotMaterialDependencies() correctly points at the material node
			// we'll produce when handling the prim, depending on the render context schema handlers.
			//
			// The only annoyance left is we must provide bForceCheckUnrealMaterialAttribute=true here for backwards compatibility with the
			// "unrealMaterial" string attribute: There is no Material prim in that case at all (so nothing for the schema handlers to be
			// invoked on), and if we do not provide the argument we'll fully miss out on these deprecated material bindings
			// We can't provide the "unreal" render context for that, because that will cause us to early resolve those bindings to the
			// UAsset content paths directly, and as said above we need to be pointing at the prim to let the material schema handlers do that
			double TimeCode = UsdUtils::GetDefaultTimeCode();
			const bool bProvideMaterialIndices = false;
			const pxr::TfToken RenderContext = pxr::UsdShadeTokens->universalRenderContext;
			UsdUtils::FUsdPrimMaterialAssignmentInfo PrototypeAssignments = UsdUtils::GetPrimMaterialAssignments(
				MeshPrim,
				TimeCode,
				bProvideMaterialIndices,
				RenderContext,
				UsdContext.CachedMeshConversionOptions.MaterialPurpose,
				UsdContext.CachedMeshConversionOptions.bForceCheckUnrealMaterialAttribute,
				UsdContext.CachedMeshConversionOptions.bRequireMaterialsHaveProvidedRenderContext
			);

			if (Info.bInsideLODVariant)
			{
				UsdContext.CachedMaterialAssignments.Add(PrimPath, PrototypeAssignments);
			}

			// Move these into the asset node because the USD Pipeline will compare these with the assigned material's
			// parameter-to-primvar mapping in order to make sure the mesh is using a primvar-compatible material.
			//
			// Note that ideally we'd cache this mapping and reuse it on the payload retrieval step. Instead, we will
			// just end up calling the same function again during payload retrieval and hoping that it produces the same
			// primvar-to-UVIndex mapping. It should though, as the mesh conversion options are the same. We can't cache
			// the mapping because we run into USD allocator issues, given that all the strings contained in the
			// FUsdPrimMaterialAssignmentInfo object are allocated inside an USD allocator scope
			for (const TPair<FString, int32>& PrimvarPair : UsdUtils::GetPrimvarToUVIndexMap(MeshPrim))
			{
				const FString& PrimvarName = PrimvarPair.Key;
				int32 UVIndex = PrimvarPair.Value;

				MeshNode->AddInt32Attribute(UE::Interchange::USD::PrimvarUVIndexAttributePrefix + PrimvarName, UVIndex);
			}

			SetSlotMaterialDependencies(MeshNode, PrototypeAssignments, AccumulatedInfo, UsdContext);
		}

		// Translate per-prim build settings API schemas as user attributes on the mesh node
		TranslateAPISchemaAttributes(MeshPrim, UnrealIdentifiers::StaticMeshBuildSettingsAPI, MeshNode);
		TranslateAPISchemaAttributes(MeshPrim, UnrealIdentifiers::NaniteBuildSettingsAPI, MeshNode);
	}

	bool GetSkeletalMeshPayloadData(
		FString PayloadKey,
		bool bBakeMeshes,
		FTransform GlobalOffsetTransform,
		UInterchangeUsdContext& UsdContext,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FRWLock& SkeletalMeshDescriptionsLock,
		TMap<FString, FMeshDescription>& InOutPayloadKeyToSkeletalMeshDescriptions,
		FMeshDescription& OutMeshDescription,
		TArray<FString>& OutJointNames,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetSkeletalMeshPayloadData)

#if WITH_EDITOR
		const bool bIsLODMesh = CheckAndChopPayloadPrefix(PayloadKey, LODPrefix);

		const FString& PrimPath = PayloadKey;
		UE::FUsdPrim Prim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*PrimPath});
		if (!Prim)
		{
			if (bIsLODMesh)
			{
				Prim = TryGettingInactiveLODPrim(PrimPath, UsdContext);
			}

			if (!Prim)
			{
				return false;
			}
		}

		const FString MeshNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);

		UsdToUnreal::FUsdMeshConversionOptions OptionsCopy = Options;

		// Read these variables from the data we cached during traversal for translation
		TSharedPtr<TArray<FString>> JointNames = nullptr;
		UE::FUsdSkelSkeletonQuery SkelQuery;
		{
			FReadScopeLock ReadLock{UsdContext.NodeUidToCachedTraversalInfoLock};

			const FTraversalInfo* MeshInfo = UsdContext.NodeUidToCachedTraversalInfo.Find(MeshNodeUid);
			if (!MeshInfo)
			{
				return false;
			}

			SkelQuery = MeshInfo->ResolveSkelQuery(UsdContext.GetUsdStage());
			if (!SkelQuery)
			{
				return false;
			}

			// The above fields are associated to the mesh *asset* node Uid (hence the prefix),
			// while the joint names are associated to the skeleton *scene* node Uid, so no prefix
			UE::FUsdPrim SkeletonPrim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath(**MeshInfo->BoundSkeletonPrimPath));
			if (!SkeletonPrim)
			{
				return false;
			}
			const FString SkeletonNodeUid = UsdContext.MakeSceneNodeUid(SkeletonPrim);
			const FTraversalInfo* SkeletonInfo = UsdContext.NodeUidToCachedTraversalInfo.Find(SkeletonNodeUid);
			if (!SkeletonInfo)
			{
				return false;
			}
			JointNames = SkeletonInfo->SkelJointUsdNames;
			if (!JointNames)
			{
				return false;
			}

			if (bBakeMeshes)
			{
				OptionsCopy.AdditionalTransform = SkeletonInfo->SceneGlobalTransform * GlobalOffsetTransform;
			}

			// Get skeletal mesh Nanite assembly data, if available
			if (MeshInfo && MeshInfo->NaniteAssemblyTraversalResult)
			{
				if (const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult* TraversalResult = MeshInfo->NaniteAssemblyTraversalResult.Get())
				{
					// Add native instance data to the output description
					UE::Interchange::USD::GetNaniteAssemblyPayloadDataForPrims(
						UsdContext.GetUsdStage(),
						OptionsCopy,
						*TraversalResult,
						SkeletonNodeUid,
						*JointNames,
						OutNaniteAssemblyDescription);

					// Add pointinstancer to the output description
					for (const UE::FSdfPath& TopLevelPointInstancerPath : TraversalResult->GetTopLevelPointInstancerPaths())
					{
						UE::FUsdPrim PointInstancerPrim = UsdContext.GetUsdStage().GetPrimAtPath(TopLevelPointInstancerPath);
						UE::Interchange::USD::GetNaniteAssemblyPayloadDataForPointInstancer(
							PointInstancerPrim, 
							OptionsCopy,
							*TraversalResult,
							SkeletonNodeUid, 
							*JointNames, 
							OutNaniteAssemblyDescription);
					}
				}
			}
		}

		// We cache these because we may need to retrieve these again when computing morph target mesh descriptions
		FWriteScopeLock WriteLock{SkeletalMeshDescriptionsLock};
		if (FMeshDescription* FoundMeshDescription = InOutPayloadKeyToSkeletalMeshDescriptions.Find(PayloadKey))
		{
			OutMeshDescription = *FoundMeshDescription;
			OutJointNames = *JointNames;
			return true;
		}

		UE::FUsdSkelSkinningQuery SkinningQuery = UsdUtils::CreateSkinningQuery(Prim, SkelQuery);
		if (!SkinningQuery)
		{
			return false;
		}

		FSkeletalMeshImportData SkelMeshImportData;
		UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;
		FMeshDescription TempMeshDescription;

		pxr::UsdSkelSkinningQuery& PxrSkelSkinningQueryPtr = SkinningQuery;
		pxr::UsdSkelSkeletonQuery& PxrSkelSkeletonQueryPtr = SkelQuery;

		bool bSuccess = UsdToUnreal::ConvertGeomMesh(
			Prim,
			TempMeshDescription,
			TempMaterialInfo,
			OptionsCopy,
			&PxrSkelSkinningQueryPtr,
			&PxrSkelSkeletonQueryPtr
		);
		if (!bSuccess)
		{
			return false;
		}

		OutMeshDescription = MoveTemp(TempMeshDescription);

		UsdUtils::FUsdPrimMaterialAssignmentInfo* AssignmentPtr = &TempMaterialInfo;
		if (bIsLODMesh)
		{
			// Use our cached material assignments instead of whatever we pull from ConvertSkinnedMesh
			// because if we're in an LOD mesh then we may be reading from a temp stage, that has a
			// population mask that may not include the material, meaning ConvertSkinnedMesh may have failed
			// to resolve all the bindings. The cached assignments come from ConvertMeshNode step, where
			// we switch the active variant on the current stage and so get nice material bindings that
			// resolve normally.
			// TODO: We may not need this anymore, as we don't use population masks to open the LOD stages anymore
			//
			// Note that we can't even use the info cache here, because it wouldn't have cached info
			// about the inactive LOD variants
			if (UsdUtils::FUsdPrimMaterialAssignmentInfo* CachedInfo = UsdContext.CachedMaterialAssignments.Find(PrimPath))
			{
				AssignmentPtr = CachedInfo;
			}
		}

		FixMaterialSlotNames(OutMeshDescription, AssignmentPtr->Slots, UsdContext);

		OutJointNames = *JointNames;

		InOutPayloadKeyToSkeletalMeshDescriptions.Add(PayloadKey, OutMeshDescription);

		return true;
#else
		return false;
#endif	  // WITH_EDITOR
	}

	bool GetMorphTargetPayloadData(
		FString PayloadKey,
		bool bBakeMeshes,
		FTransform GlobalOffsetTransform,
		UInterchangeUsdContext& UsdContext,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FRWLock& SkeletalMeshDescriptionsLock,
		TMap<FString, FMeshDescription>& InOutPayloadKeyToSkeletalMeshDescriptions,
		FMeshDescription& OutMeshDescription,
		FString& OutMorphTargetName
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetMorphTargetPayloadData)

		bool bIsLODMesh = false;
		FString MeshPrimPath;
		int32 BlendShapeIndex = INDEX_NONE;
		FString InbetweenName;
		if (!ParseMorphTargetMeshPayloadKey(PayloadKey, bIsLODMesh, MeshPrimPath, BlendShapeIndex, InbetweenName))
		{
			return false;
		}

		UE::FUsdPrim MeshPrim = UsdContext.GetUsdStage().GetPrimAtPath(FSdfPath{*MeshPrimPath});
		if (!MeshPrim)
		{
			if (bIsLODMesh)
			{
				MeshPrim = TryGettingInactiveLODPrim(MeshPrimPath, UsdContext);
			}

			if (!MeshPrim)
			{
				return false;
			}
		}

		UE::FUsdSkelBlendShapeQuery Query{MeshPrim};
		if (!Query)
		{
			return false;
		}

		UE::FUsdSkelBlendShape BlendShape = Query.GetBlendShape(BlendShapeIndex);
		if (!BlendShape)
		{
			return false;
		}

		TArray<FString> UnusedJointNames;
		TOptional<UE::Interchange::FNaniteAssemblyDescription> UnusedNaniteData;
		const FString MeshPayloadKey = bIsLODMesh ? LODPrefix + MeshPrimPath : MeshPrimPath;
		bool bConverted = GetSkeletalMeshPayloadData(
			MeshPayloadKey,
			bBakeMeshes,
			GlobalOffsetTransform,
			UsdContext,
			Options,
			SkeletalMeshDescriptionsLock,
			InOutPayloadKeyToSkeletalMeshDescriptions,
			OutMeshDescription,
			UnusedJointNames,
			UnusedNaniteData
		);
		if (!bConverted || OutMeshDescription.IsEmpty())
		{
			return false;
		}

		OutMorphTargetName = BlendShape.GetPrim().GetName().ToString();
		if (!InbetweenName.IsEmpty())
		{
			OutMorphTargetName += TEXT("_") + InbetweenName;
		}

		// Collect GeomBindTransform if we have one
		FMatrix GeomBindTransform = FMatrix::Identity;
		{
			UE::FUsdSkelSkeletonQuery SkelQuery;
			{
				FReadScopeLock ReadLock{UsdContext.NodeUidToCachedTraversalInfoLock};

				const FString MeshNodeUid = UsdContext.MakeAssetNodeUid(MeshPrim, MeshPrefix);
				const FTraversalInfo* MeshInfo = UsdContext.NodeUidToCachedTraversalInfo.Find(MeshNodeUid);
				if (MeshInfo)
				{
					SkelQuery = MeshInfo->ResolveSkelQuery(UsdContext.GetUsdStage());
				}
			}

			if (SkelQuery)
			{
				if (UE::FUsdSkelSkinningQuery SkinningQuery = UsdUtils::CreateSkinningQuery(MeshPrim, SkelQuery))
				{
					GeomBindTransform = SkinningQuery.GetGeomBindTransform(Options.TimeCode.GetValue());
				}
			}
		}

		const float Weight = 1.0f;
		return UsdUtils::ApplyBlendShape(
			OutMeshDescription,
			BlendShape.GetPrim(),
			GeomBindTransform,
			Options.AdditionalTransform,
			Weight,
			InbetweenName
		);
	}
}	 // namespace UE::GeomGprimSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FGprimSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("GprimHandler");
		return HandlerName;
	}

	const FString& FGprimSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Gprim");
		return SchemaName;
	}

	bool FGprimSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGprimSchemaHandler::CanHandlePrim)

		// We have to do some precise filtering here, because while we do want this handler to cover the Mesh and primitive schemas
		// (for which the common base is the UsdGeomGprim schema), we don't want to handle *all* Gprims, as that includes UsdVolVolume,
		// UsdGeomBasisCurves, nurbs, points, custom user schemas, etc.
		return Prim.IsA(TEXT("Mesh")) ||
			   Prim.IsA(TEXT("Capsule")) ||
			   Prim.IsA(TEXT("Capsule_1")) ||
			   Prim.IsA(TEXT("Cone")) ||
			   Prim.IsA(TEXT("Cube")) ||
			   Prim.IsA(TEXT("Cylinder")) ||
			   Prim.IsA(TEXT("Cylinder_1")) ||
			   Prim.IsA(TEXT("Plane")) ||
			   Prim.IsA(TEXT("Sphere"));
	}

	TOptional<bool> FGprimSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		if (UsdUtils::IsCollisionEnabledForPrim(Prim) || UsdUtils::IsCollisionMesh(Prim))
		{
			return false;
		}

		return {};
	}

	TOptional<bool> FGprimSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// A gprim is supposed to be a leaf node in the scene graph, so there shouldn't even be anything to collapse below us
		return false;
	}

	bool FGprimSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGprimSchemaHandler::OnTranslate)

		using namespace UE::GeomGprimSchemaHandler::Private;

		const bool bIsSkinned = static_cast<bool>(TraversalInfo.ClosestParentSkelRootPath) && Prim.HasAPI(TEXT("SkelBindingAPI"));
		const bool bIsPrimitiveShape = !Prim.IsA(TEXT("Mesh")); // See CanHandlePrim()
		const bool bIsAnimated = UsdUtils::IsAnimatedMesh(Prim);

		// Don't process mesh prims with disabled purposes
		if (UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings())
		{
			if (!EnumHasAllFlags(static_cast<EUsdPurpose>(Settings->GeometryPurpose), IUsdPrim::GetPurpose(Prim)))
			{
				return false;
			}
		}

		if (Prim.GetParent().IsA(TEXT("Gprim")))
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"NestedGprims",
					"Prim '{0}' is a Gprim child of another Gprim. This configuration is not recommended or supported very well, and may lead to unexpected results."
				),
				FText::FromString(Prim.GetPrimPath().GetString())
			));
		}

		const FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);
		const FString NewNodeName{Prim.GetName().ToString()};
		UInterchangeMeshNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeMeshNode>(
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, Prim.GetPrimPath().GetString());

		ConvertMeshNode(Prim, bIsSkinned, bIsPrimitiveShape, bIsAnimated, AssetNode, TraversalInfo, AccumulatedInfo, UsdContext);

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			SceneNode->SetCustomAssetInstanceUid(AssetNode->GetUniqueID());

			// Set component material overrides
			//
			// For now we only need material overrides in case of USD instanceables.
			//
			// TODO: While the USD translator and pipeline's side of this should be working, material overrides don't quite work just yet,
			// as ActorHelper::ApplySlotMaterialDependencies (the thing responsible for setting material overrides on the actor components)
			// will try to set these overrides according to slot name, while we always just place the (mesh's) material path itself as the
			// slot name... We need some different mechanism for setting slot name (maybe a translator/pipeline setting to just use the
			// GeomSubset prim name, or the slot index?) that can also work for setting material overrides. Also, see FixMaterialSlotNames()
			if (Prim.IsInstance() || Prim.IsInstanceProxy())
			{
				// This is in charge of picking up on collection-based material assignments, as we'll use the instance proxy prim here
				//
				// See the large comment on the other call to GetPrimMaterialAssignments, from within FGprimSchemaHandler::OnTranslate()
				const double TimeCode = UsdUtils::GetDefaultTimeCode();
				const bool bProvideMaterialIndices = false;
				const pxr::TfToken RenderContext = pxr::UsdShadeTokens->universalRenderContext;
				UsdUtils::FUsdPrimMaterialAssignmentInfo InstanceAssignments = UsdUtils::GetPrimMaterialAssignments(
					Prim,
					TimeCode,
					bProvideMaterialIndices,
					RenderContext,
					UsdContext.CachedMeshConversionOptions.MaterialPurpose,
					UsdContext.CachedMeshConversionOptions.bForceCheckUnrealMaterialAttribute,
					UsdContext.CachedMeshConversionOptions.bRequireMaterialsHaveProvidedRenderContext
				);

				SetSlotMaterialDependencies(SceneNode, InstanceAssignments, AccumulatedInfo, UsdContext);
			}
		}

		// The strong recommendation is for Gprims to be leaf nodes, so let's not recurse into children.
		// UsdGeomSubsets will be manually handled when calling GetPrimMaterialAssignments()
		constexpr bool bTraverseInstanceProxies = true;
		for (const FUsdPrim& ChildPrim : Prim.GetFilteredChildren(bTraverseInstanceProxies))
		{
			UsdContext.HandledPrimInfo.FindOrAdd(ChildPrim.GetPrimPath());
		}

		return true;
	}

	bool FGprimSchemaHandler::OnGetMeshPayloadData(
		const FInterchangeMeshPayLoadKey& PayloadKey,
		const UE::Interchange::FAttributeStorage& PayloadAttributes,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FMeshPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGprimSchemaHandler::OnGetMeshPayloadData)

		using namespace UE::GeomGprimSchemaHandler::Private;
		using namespace UE::Interchange::USD;

		bool bSuccess = false;

		UsdToUnreal::FUsdMeshConversionOptions OptionsCopy = UsdContext.CachedMeshConversionOptions;
		OptionsCopy.bMergeIdenticalMaterialSlots = false;	 // This must always be false, because we need the material assignments we read from the
		// meshes to match up with whatever we cached from ConvertMeshNode, in order to fixup LOD
		// material slots

		static_assert(
			uint8(EInterchangeUsdPrimvar::All) == UsdToUnreal::FUsdMeshConversionOptions::EImportPrimvar::All,
			"FUsdMeshConversionOptions::EImportPrimvar::All is different from EInterchangeUsdPrimvar::All"
		);
		static_assert(
			uint8(EInterchangeUsdPrimvar::Bake) == UsdToUnreal::FUsdMeshConversionOptions::EImportPrimvar::Bake,
			"FUsdMeshConversionOptions::EImportPrimvar::Bake is different from EInterchangeUsdPrimvar::Bake"
		);
		static_assert(
			uint8(EInterchangeUsdPrimvar::Standard) == UsdToUnreal::FUsdMeshConversionOptions::EImportPrimvar::Standard,
			"FUsdMeshConversionOptions::EImportPrimvar::Standard is different from EInterchangeUsdPrimvar::Standard"
		);

		PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::SubdivisionLevelAttributeKey}, OptionsCopy.SubdivisionLevel);
		PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Import}, reinterpret_cast<int32&>(OptionsCopy.ImportPrimvars));

		if (EInterchangeUsdPrimvar(OptionsCopy.ImportPrimvars) != EInterchangeUsdPrimvar::Standard)
		{
			if (int32 PrimvarNumber; PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Number}, PrimvarNumber)
									 == EAttributeStorageResult::Operation_Success)
			{
				OptionsCopy.PrimvarNames.Reserve(PrimvarNumber);
				for (int32 Index = 0; Index < PrimvarNumber; ++Index)
				{
					if (FString PrimvarName;
						PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{USD::Primvar::Name + FString::FromInt(Index)}, PrimvarName)
						== EAttributeStorageResult::Operation_Success)
					{
						OptionsCopy.PrimvarNames.Emplace(MoveTemp(PrimvarName));
					}
				}
			}
		}

		UE::Interchange::FMeshPayloadData MeshPayloadData;
		switch (PayloadKey.Type)
		{
			case EInterchangeMeshPayLoadType::STATIC:
			{
				PayloadAttributes.GetAttribute(
					UE::Interchange::FAttributeKey{MeshPayload::Attributes::MeshGlobalTransform},
					OptionsCopy.AdditionalTransform
				);

				bSuccess = GetStaticMeshPayloadData(
					PayloadKey.UniqueId,
					UsdContext,
					OptionsCopy,
					MeshPayloadData.MeshDescription,
					MeshPayloadData.NaniteAssemblyDescription
				);
				break;
			}
			case EInterchangeMeshPayLoadType::SKELETAL:
			{
				// Don't use MeshGlobalTransform here as that will be the scene transform of our Mesh prims, which is not relevant for USD skinning.
				// With baking, we want to first apply geomBindTransform, and then apply the skeleton's localToWorld transform. ConvertGeomMesh can
				// sort out the geomBindTransform (which should always be applied), so here we set the baking transform to the skeleton prim's
				// transform if needed
				bool bBakeMeshes = false;
				FTransform GlobalOffsetTransform;
				PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{MeshPayload::Attributes::BakeMeshes}, bBakeMeshes);
				PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{MeshPayload::Attributes::GlobalOffsetTransform}, GlobalOffsetTransform);

				// The skeletal mesh factory will already remap our bones by name from RetrieveAllSkeletalMeshPayloads() anyway,
				// so we shouldn't offset anything ourselves
				OptionsCopy.bOffsetJointIndicesForMultipleRootBones = false;

				bSuccess = GetSkeletalMeshPayloadData(
					PayloadKey.UniqueId,
					bBakeMeshes,
					GlobalOffsetTransform,
					UsdContext,
					OptionsCopy,
					SkeletalMeshDescriptionsLock,
					PayloadKeyToSkeletalMeshDescriptions,
					MeshPayloadData.MeshDescription,
					MeshPayloadData.JointNames,
					MeshPayloadData.NaniteAssemblyDescription
				);
				break;
			}
			case EInterchangeMeshPayLoadType::MORPHTARGET:
			{
				// See the ::SKELETAL case
				bool bBakeMeshes = false;
				FTransform GlobalOffsetTransform;
				PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{MeshPayload::Attributes::BakeMeshes}, bBakeMeshes);
				PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{MeshPayload::Attributes::GlobalOffsetTransform}, GlobalOffsetTransform);

				bSuccess = GetMorphTargetPayloadData(
					PayloadKey.UniqueId,
					bBakeMeshes,
					GlobalOffsetTransform,
					UsdContext,
					OptionsCopy,
					SkeletalMeshDescriptionsLock,
					PayloadKeyToSkeletalMeshDescriptions,
					MeshPayloadData.MeshDescription,
					MeshPayloadData.MorphTargetName
				);
				break;
			}
			case EInterchangeMeshPayLoadType::NONE:	   // Fallthrough
			default:
				break;
		}

		if (bSuccess)
		{
			InOutPayloadData.Emplace(MeshPayloadData);
		}

		return bSuccess;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
