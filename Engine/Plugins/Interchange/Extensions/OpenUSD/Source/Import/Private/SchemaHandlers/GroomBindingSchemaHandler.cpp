// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/GroomBindingSchemaHandler.h"

#include "InterchangeGroomBindingNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDIntegrationUtils.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdStage.h"

namespace UE::Interchange::USD
{
	const FString& FGroomBindingSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("GroomBindingHandler");
		return HandlerName;
	}

	const FString& FGroomBindingSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Mesh");
		return SchemaName;
	}

	bool FGroomBindingSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingSchemaHandler::CanHandlePrim)
		return (Prim.IsA(TEXT("Mesh")) || Prim.IsA(TEXT("SkelRoot"))) && Prim.HasAPI(TEXT("GroomBindingAPI"));
	}

	TOptional<bool> FGroomBindingSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FGroomBindingSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FGroomBindingSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingSchemaHandler::OnTranslate)
		const bool bIsSkinned = static_cast<bool>(TraversalInfo.ClosestParentSkelRootPath) && Prim.HasAPI(TEXT("SkelBindingAPI"));
		const bool bIsMesh = Prim.IsA(TEXT("Mesh"));
		const bool bIsAnimated = UsdUtils::IsAnimatedMesh(Prim);

		if (!bIsSkinned && !(bIsMesh && bIsAnimated))
		{
			return false;
		}
		
		UE::FUsdStage Stage = Prim.GetStage();

		FString GroomPrimPath;
		if (FUsdRelationship Relationship = Prim.GetRelationship(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealGroomToBind)))
		{
			TArray<FSdfPath> Targets;
			Relationship.GetTargets(Targets);

			if (Targets.Num() > 0)
			{
				// Validate that the target prim is in fact a groom prim
				const FSdfPath& TargetPrimPath = Targets[0];
				FUsdPrim TargetPrim = Stage.GetPrimAtPath(TargetPrimPath);
				if (TargetPrim && UsdUtils::PrimHasSchema(TargetPrim, UnrealIdentifiers::GroomAPI))
				{
					GroomPrimPath = TargetPrimPath.GetString();
				}
			}
		}

		UE::FUsdPrim GroomPrim = Stage.GetPrimAtPath(UE::FSdfPath{*GroomPrimPath});
		if (!GroomPrim)
		{
			return false;
		}

		const FString NodeUid = UsdContext.MakeAssetNodeUid(Prim, GroomBindingPrefix);
		const FString NodeName = FString::Printf(TEXT("%s_%s_Binding"), *GroomPrim.GetName().ToString(), *Prim.GetName().ToString());
		bool bCreated = false;
		UInterchangeGroomBindingNode* GroomBindingAssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeGroomBindingNode>(*UsdContext.GetNodeContainer(), NodeUid, NodeName, &bCreated);
		if (!GroomBindingAssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*GroomBindingAssetNode, Prim.GetPrimPath().GetString());		

		if (bIsMesh)
		{
			const FString MeshNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);
			GroomBindingAssetNode->SetTargetMeshDependencyUid(MeshNodeUid);
		}
		else
		{
			// Find mesh prim under SkelRoot
			TArray<FUsdPrim> Children = Prim.GetChildren();
			for (const FUsdPrim& Child : Children)
			{
				if (Child.IsA(TEXT("Mesh")))
				{
					const FString MeshNodeUid = UsdContext.MakeAssetNodeUid(Child, MeshPrefix);
					GroomBindingAssetNode->SetTargetMeshDependencyUid(MeshNodeUid);
					break;
				}
			}

		}

		FString GroomUid = UsdContext.MakeAssetNodeUid(GroomPrim, GroomPrefix);
		GroomBindingAssetNode->SetGroomDependencyUid(GroomUid);

		return true;
	}
}
#endif	  // USE_USD_SDK