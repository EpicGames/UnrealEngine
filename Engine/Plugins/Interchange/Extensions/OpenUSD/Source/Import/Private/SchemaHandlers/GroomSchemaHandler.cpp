// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/GroomSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDGroomConversion.h"
#include "UsdWrappers/UsdPrim.h"

#include "Groom/InterchangeGroomPayloadData.h"
#include "InterchangeGroomNode.h"

#include "GroomCacheData.h"

namespace UE::Interchange::USD
{
	const FString& FGroomSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("GroomHandler");
		return HandlerName;
	}

	const FString& FGroomSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("BasisCurves");
		return SchemaName;
	}

	bool FGroomSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomSchemaHandler::CanHandlePrim)

		return (Prim.IsA(TEXT("BasisCurves")) || Prim.IsA(TEXT("Xform"))) && Prim.HasAPI(TEXT("GroomAPI"));
	}

	TOptional<bool> FGroomSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FGroomSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FGroomSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomSchemaHandler::OnTranslate)

		// Check if there's any actual groom data
		FGroomAnimationInfo AnimInfo;
		FHairDescription HairDescription;
		if (!UsdToUnreal::ConvertGroomHierarchy(Prim, UsdUtils::GetEarliestTimeCode(), FTransform::Identity, HairDescription, &AnimInfo))
		{
			return false;
		}

		double StageTimeCodesPerSecond = Prim.GetStage().GetTimeCodesPerSecond();
		if (StageTimeCodesPerSecond == 0.0)
		{
			StageTimeCodesPerSecond = 24.0;
		}

		AnimInfo.SecondsPerFrame = static_cast<float>(1 / StageTimeCodesPerSecond);

		const bool bIsValidGroomCache = AnimInfo.IsValid();

		FString PrimPath = Prim.GetPrimPath().GetString();
		FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, GroomPrefix);
		FString NewNodeName{Prim.GetName().ToString()};
		UInterchangeGroomNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeGroomNode>(
			*UsdContext.GetNodeContainer(), 
			NewNodeUid, 
			NewNodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, PrimPath);

		AssetNode->SetPayloadKey(PrimPath, bIsValidGroomCache ? EInterchangeGroomPayLoadType::ANIMATED : EInterchangeGroomPayLoadType::STATIC);

		if (bIsValidGroomCache)
		{
			AssetNode->SetCustomNumFrames(AnimInfo.NumFrames);
			AssetNode->SetCustomStartFrame(AnimInfo.StartFrame);
			AssetNode->SetCustomEndFrame(AnimInfo.EndFrame);
			AssetNode->SetCustomFrameRate(StageTimeCodesPerSecond);

			static_assert((int)EGroomCacheAttributes::None == (int)EInterchangeGroomCacheAttributes::None);
			static_assert((int)EGroomCacheAttributes::Position == (int)EInterchangeGroomCacheAttributes::Position);
			static_assert((int)EGroomCacheAttributes::Width == (int)EInterchangeGroomCacheAttributes::Width);
			static_assert((int)EGroomCacheAttributes::Color == (int)EInterchangeGroomCacheAttributes::Color);

			AssetNode->SetCustomGroomCacheAttributes(static_cast<EInterchangeGroomCacheAttributes>(AnimInfo.Attributes));
		}

		return true;
	}

	bool FGroomSchemaHandler::OnGetGroomPayloadData(
		const FInterchangeGroomPayloadKey& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FGroomPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomSchemaHandler::OnGetGroomPayloadData)

		const FString& PrimPath = PayloadKey.UniqueId;
		UE::FUsdPrim Prim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*PrimPath});

		FGroomAnimationInfo AnimInfo;
		UE::Interchange::FGroomPayloadData PayloadData;
		const double TimeCode = PayloadKey.Type == EInterchangeGroomPayLoadType::ANIMATED ? PayloadKey.FrameNumber : UsdUtils::GetEarliestTimeCode();

		const bool bSuccess = UsdToUnreal::ConvertGroomHierarchy(Prim, TimeCode, FTransform::Identity, PayloadData.HairDescription, &AnimInfo);

		if (bSuccess && PayloadData.HairDescription.IsValid())
		{
			InOutPayloadData = PayloadData;
			return true;
		}

		return false;
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
