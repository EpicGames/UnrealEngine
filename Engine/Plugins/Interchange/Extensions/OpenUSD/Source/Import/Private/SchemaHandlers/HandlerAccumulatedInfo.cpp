// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/HandlerAccumulatedInfo.h"

#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "USDErrorUtils.h"

#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

namespace UE::Interchange::USD
{
	UInterchangeBaseNode* FHandlerAccumulatedInfo::GetMainSceneNode()
	{
		if (PrimSceneNodes.Num() > 0)
		{
			return PrimSceneNodes[0];
		}

		return nullptr;
	}

	UInterchangeBaseNode* FHandlerAccumulatedInfo::GetOrCreateMainSceneNode(
		const UE::FUsdPrim& Prim,
		const FTraversalInfo& TraversalInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		// The main scene node is the first one in the list, always (even if nullptr)
		if (PrimSceneNodes.Num() > 0)
		{
			return PrimSceneNodes[0];
		}

		UInterchangeSceneNode* NewSceneNode = GetOrCreateDefaultSceneNode(Prim, TraversalInfo, UsdContext);
		if (ensure(NewSceneNode))
		{
			PrimSceneNodes.Add(NewSceneNode);
		}

		return NewSceneNode;
	}

	UInterchangeBaseNode* FHandlerAccumulatedInfo::GetOrCreateAssetNode(
		const TSubclassOf<UInterchangeBaseNode>& NodeClass,
		UInterchangeBaseNodeContainer& NodeContainer,
		const FString& NewNodeUid,
		const FString& NewNodeDisplayName,
		bool* bOutCreatedNode
	)
	{
		if (bOutCreatedNode)
		{
			*bOutCreatedNode = false;
		}

		for (UInterchangeBaseNode* AssetNode : PrimAssetNodes)
		{
			if (AssetNode->IsA(NodeClass))
			{
				return AssetNode;
			}
		}

		// Check if there's another node in the node container with the UID we can reuse and adopt for this prim.
		// Not entirely sure whether this is a good idea or not, but it could help as a failsafe when other translators produces our nodes,
		// as if we can find a node with the desired ID in the node container then it likely should be on PrimAssetNodes anyway.
		// FHandlerAccumulatedInfo info is only used during translation, so it's fine to be getting non-const access to translated nodes
		UInterchangeBaseNode* ExistingNode = const_cast<UInterchangeBaseNode*>(NodeContainer.GetNode(NewNodeUid));
		if (ExistingNode)
		{
			if (ExistingNode->IsA(NodeClass))
			{
				PrimAssetNodes.Add(ExistingNode);
				return ExistingNode;
			}
			else
			{
				USD_LOG_WARNING(
					TEXT(
						"Tried to create a '%s' node with UID '%s', but encountered incompatible node of class '%s' with the same UID already in the node container! The node may not be translated correctly."
					),
					*NodeClass->GetClassPathName().ToString(),
					*NewNodeUid,
					*ExistingNode->GetClass()->GetPathName()
				);
				return nullptr;
			}
		}

		// Create a brand new node
		UInterchangeBaseNode* NewNode = NewObject<UInterchangeBaseNode>(&NodeContainer, NodeClass);
		NodeContainer.SetupNode(NewNode, NewNodeUid, NewNodeDisplayName, EInterchangeNodeContainerType::TranslatedAsset);
		NewNode->SetAssetName(NewNodeDisplayName);
		if (bOutCreatedNode)
		{
			*bOutCreatedNode = true;
		}
		PrimAssetNodes.Add(NewNode);
		return NewNode;
	}

	UInterchangeBaseNode* FHandlerAccumulatedInfo::GetAssetNodeOfClass(const TSubclassOf<UInterchangeBaseNode>& NodeClass)
	{
		for (UInterchangeBaseNode* AssetNode : PrimAssetNodes)
		{
			if (AssetNode->IsA(NodeClass))
			{
				return AssetNode;
			}
		}

		return nullptr;
	}

	void FHandlerAccumulatedInfo::AppendInfo(const FHandlerAccumulatedInfo& Other)
	{
		PrimSceneNodes.Append(Other.PrimSceneNodes);
		PrimAssetNodes.Append(Other.PrimAssetNodes);
	}

	void FHandlerAccumulatedInfo::AppendInfo(FHandlerAccumulatedInfo&& Other)
	{
		PrimSceneNodes.Append(MoveTemp(Other.PrimSceneNodes));
		PrimAssetNodes.Append(MoveTemp(Other.PrimAssetNodes));
	}	
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
