// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/ImageableSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDErrorUtils.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdEditContext.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdTyped.h"
#include "UsdWrappers/UsdVariantSets.h"

#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Async/ParallelFor.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "ImageableSchemaHandler"

namespace UE::GeomImageableSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	void HandleVariantStyleLODs(
		const UE::FUsdPrim& Prim,
		UInterchangeSceneNode* ParentSceneNode,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HandleVariantStyleLODs)

		if (!ParentSceneNode || !Prim)
		{
			return;
		}

		UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator();
		if (!CurrentTranslator)
		{
			return;
		}

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		UE::FUsdVariantSets VariantSets = Prim.GetVariantSets();
		const UE::FUsdVariantSet LODVariantSet = VariantSets.GetVariantSet(UE::Interchange::USD::LODString);
		if (!LODVariantSet.IsValid())
		{
			return;
		}

		const FString ActiveVariant = LODVariantSet.GetVariantSelection();
		if (!IsValidLODName(ActiveVariant))
		{
			return;
		}

		UE::FUsdStage UsdStage = UsdContext.GetUsdStage();

		FTraversalInfo InfoCopy = TraversalInfo;
		InfoCopy.ParentNode = ParentSceneNode;
		InfoCopy.bInsideLODVariant = true;

		const FString LODContainerUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(ParentSceneNode->GetUniqueID());
		UInterchangeMeshLODContainerNode* LODContainerNode = GetExistingNode<UInterchangeMeshLODContainerNode>(*NodeContainer, LODContainerUid);
		if (!LODContainerNode)
		{
			LODContainerNode = NewObject<UInterchangeMeshLODContainerNode>(NodeContainer);
			NodeContainer->SetupNode(
				LODContainerNode,
				LODContainerUid,
				ParentSceneNode->GetDisplayLabel(),
				EInterchangeNodeContainerType::TranslatedAsset
			);
		}

		TArray<UInterchangeSceneNode*> LODMeshSceneNodes;

		bool bSwitchedFromInitialVariant = false;
		TSet<int32> ProcessedLODIndices;
		for (const FString& VariantName : LODVariantSet.GetVariantNames())
		{
			if (!IsValidLODName(VariantName))
			{
				continue;
			}

			{
				// For creating the scene nodes themselves we'll switch the active variant on the currently opened stage
				// (still using the session layer to minimize impact to the actual layer). This mainly so that we can retrieve
				// and fetch and cache the correct material bindings for the LOD meshes. Later on we'll use separate stages
				// with population masks to read the LODs concurrently, and we won't be able to resolve material bindings
				UE::FUsdEditContext Context{UsdStage, UsdStage.GetSessionLayer()};

				bool bSwitchedVariant = VariantSets.SetSelection(UE::Interchange::USD::LODString, VariantName);
				if (!bSwitchedVariant)
				{
					continue;
				}
				bSwitchedFromInitialVariant = true;
			}

			if (UE::FUsdPrim LODPrim = GetLODPrim(Prim, VariantName))
			{
				int32 LODIndex = GetLODIndexFromName(VariantName);
				if (ProcessedLODIndices.Contains(LODIndex))
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT("LodSubtreeDuplicateLOD", "Ignoring duplicate LOD Index {0} found in Prim '{1}'."),
						LODIndex,
						FText::FromString(Prim.GetPrimPath().GetString())
					));
					continue;
				}

				FHandlerAccumulatedInfo MeshPrimAccumulatedInfo;

				FString PrimPath = LODPrim.GetPrimPath().GetString();
				const bool bAllowSceneNodeGeneration = false;
				CurrentTranslator->TranslatePrimSubtree(LODPrim, InfoCopy, bAllowSceneNodeGeneration, &MeshPrimAccumulatedInfo);
					
				ProcessedLODIndices.Add(LODIndex);

				for (UInterchangeBaseNode* BaseNode : MeshPrimAccumulatedInfo.PrimAssetNodes)
				{
					if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode); MeshNode && !MeshNode->IsMorphTarget())
					{
						FString OriginalMeshPrimPath = UE::Interchange::USD::GetMeshPrimPath(*MeshNode);

						FTransform MeshTransform = UE::Interchange::USD::GetCombinedTransform(LODPrim, LODPrim.GetStage().GetPrimAtPath(UE::FSdfPath{ *OriginalMeshPrimPath }));

						LODContainerNode->AddMeshForLODIndex(LODIndex, MeshNode->GetUniqueID(), MeshTransform);
					}
				}
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"InvalidLodMeshName",
						"Failed to parse a LOD Mesh from variant '{0}' of prim '{1}'. For automatic parsing of LODs, make sure there is a single prim within the variant, named exactly as the variant itself (e.g. 'LOD0', 'LOD1', etc.)"
					),
					FText::FromString(VariantName),
					FText::FromString(Prim.GetPrimPath().GetString())
				));
			}
		}

		// Put the active variant back to what it originally was
		if (bSwitchedFromInitialVariant)
		{
			{
				UE::FUsdEditContext EditContext{UsdStage, UsdStage.GetSessionLayer()};

				const bool bRestoredSelection = VariantSets.SetSelection(UE::Interchange::USD::LODString, ActiveVariant);
				ensure(bRestoredSelection);
			}

			// Recompute our skel cache here if we have any as ancestor, because switching variants could have
			// invalidated some of its internal state about its descendant prims, which we'll need to be OK
			// when handling the payloads
			TraversalInfo.RepopulateSkelCache(UsdStage);
		}

		const FString LODContainerInstanceUid = ParentSceneNode->GetUniqueID() + LODContainerInstanceSuffix;
		UInterchangeSceneNode* LODContainerInstanceNode = GetExistingNode<UInterchangeSceneNode>(*NodeContainer, LODContainerInstanceUid);
		if (!LODContainerInstanceNode)
		{
			LODContainerInstanceNode = NewObject<UInterchangeSceneNode>(NodeContainer);

			NodeContainer->SetupNode(
				LODContainerInstanceNode,
				LODContainerInstanceUid,
				ParentSceneNode->GetDisplayLabel(),
				EInterchangeNodeContainerType::TranslatedScene,
				ParentSceneNode->GetUniqueID()
			);
		}

		LODContainerInstanceNode->SetCustomAssetInstanceUid(LODContainerNode->GetUniqueID());
		AccumulatedInfo.PrimSceneNodes.Add(LODContainerInstanceNode);
	}

	void HandleLodSubtreeSchemaLODs(
		const UE::FUsdPrim& Prim,
		UInterchangeSceneNode* ParentSceneNode,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		if (!ParentSceneNode || !Prim)
		{
			return;
		}

		UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator();
		if (!CurrentTranslator)
		{
			return;
		}

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		UE::FUsdRelationship LevelsRel = Prim.GetRelationship(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLodSubtreeLevels));
		if (!LevelsRel)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"LodSubtreeMissingRelationship",
					"LOD subtree '{0}' is missing required relationship '{1}'"
				),
				FText::FromString(Prim.GetPrimPath().GetString()),
				FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLodSubtreeLevels))
			));
			return;
		}

		TArray<UE::FSdfPath> Targets;
		if (!LevelsRel.GetTargets(Targets) || Targets.IsEmpty())
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("LodSubtreeNoTargets", "LOD subtree '{0}' has no LODs specified by relationship '{1}'"),
				FText::FromString(Prim.GetPrimPath().GetString()),
				FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLodSubtreeLevels))
			));
			return;
		}

		FTraversalInfo InfoCopy = TraversalInfo;
		InfoCopy.ParentNode = ParentSceneNode;

		TMap<int32, TPair<FUsdPrim, FHandlerAccumulatedInfo>> AccumulatedInfoPerLODIndex;
		AccumulatedInfoPerLODIndex.Reserve(Targets.Num());
		for (int32 LODIndex = 0; LODIndex < Targets.Num(); LODIndex++)
		{
			const UE::FSdfPath& TargetPath = Targets[LODIndex];

			// Members must be direct children
			if (TargetPath.GetParentPath() != Prim.GetPrimPath())
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("LodSubtreeNotDirectChild", "Ignoring LOD '{0}' that is not a direct child of '{1}'"),
					FText::FromString(TargetPath.GetString()),
					FText::FromString(Prim.GetPrimPath().GetString())
				));
				continue;
			}

			FUsdPrim SubtreePrim = Prim.GetStage().GetPrimAtPath(TargetPath);
			if (!SubtreePrim)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("LodSubtreeMissingLod", "Ignoring invalid or missing LOD '{0}' specified by '{1}'"),
					FText::FromString(TargetPath.GetString()),
					FText::FromString(Prim.GetPrimPath().GetString())
				));
				continue;
			}

			FHandlerAccumulatedInfo SubtreeAccumulatedInfo;
			const bool bAllowSceneNodeGeneration = false;
			CurrentTranslator->TranslatePrimSubtree(SubtreePrim, InfoCopy, bAllowSceneNodeGeneration, &SubtreeAccumulatedInfo);

			AccumulatedInfoPerLODIndex.Add(LODIndex, TPair<FUsdPrim, FHandlerAccumulatedInfo>(SubtreePrim, SubtreeAccumulatedInfo));
		}

		//Check if at least 1 of the LODIndices contain a mesh.
		bool bParsedLODMeshes = false;
		for (const TPair<int32, TPair<FUsdPrim, FHandlerAccumulatedInfo>>& Entry : AccumulatedInfoPerLODIndex)
		{
			for (UInterchangeBaseNode* BaseNode : Entry.Value.Value.PrimAssetNodes)
			{
				if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode))
				{
					bParsedLODMeshes = true;
					break;
				}
			}
			if (bParsedLODMeshes)
			{
				break;
			}
		}

		if (bParsedLODMeshes)
		{
			//Also add the LodContainerNode that should be referenced by LODContainerInstanceNode.
			const FString LODContainerUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(ParentSceneNode->GetUniqueID());
			UInterchangeMeshLODContainerNode* LODContainerNode = GetExistingNode<UInterchangeMeshLODContainerNode>(*NodeContainer, LODContainerUid);
			if (!LODContainerNode)
			{
				LODContainerNode = NewObject<UInterchangeMeshLODContainerNode>(NodeContainer);
				NodeContainer->SetupNode(
					LODContainerNode,
					LODContainerUid,
					ParentSceneNode->GetDisplayLabel(),
					EInterchangeNodeContainerType::TranslatedAsset
				);
			}

			for (const TPair<int32, TPair<FUsdPrim, FHandlerAccumulatedInfo>>& MeshPrimAccumulatedInfoPerLODIndex : AccumulatedInfoPerLODIndex)
			{
				for (UInterchangeBaseNode* BaseNode : MeshPrimAccumulatedInfoPerLODIndex.Value.Value.PrimAssetNodes)
				{
					if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode); MeshNode && !MeshNode->IsMorphTarget())
					{
						FString OriginalMeshPrimPath = UE::Interchange::USD::GetMeshPrimPath(*MeshNode);

						FTransform MeshTransform = UE::Interchange::USD::GetCombinedTransform(
							MeshPrimAccumulatedInfoPerLODIndex.Value.Key, 
							MeshPrimAccumulatedInfoPerLODIndex.Value.Key.GetStage().GetPrimAtPath(UE::FSdfPath{ *OriginalMeshPrimPath }));

						LODContainerNode->AddMeshForLODIndex(MeshPrimAccumulatedInfoPerLODIndex.Key, MeshNode->GetUniqueID(), MeshTransform);
					}
				}
			}

			const FString LODContainerInstanceUid = ParentSceneNode->GetUniqueID() + LODContainerInstanceSuffix;
			UInterchangeSceneNode* LODContainerInstanceNode = GetExistingNode<UInterchangeSceneNode>(*NodeContainer, LODContainerInstanceUid);
			if (!LODContainerInstanceNode)
			{
				LODContainerInstanceNode = NewObject<UInterchangeSceneNode>(NodeContainer);
				NodeContainer->SetupNode(
					LODContainerInstanceNode,
					LODContainerInstanceUid,
					ParentSceneNode->GetDisplayLabel(),
					EInterchangeNodeContainerType::TranslatedScene,
					ParentSceneNode->GetUniqueID()
				);
			}

			LODContainerInstanceNode->SetCustomAssetInstanceUid(LODContainerNode->GetUniqueID());
			AccumulatedInfo.PrimSceneNodes.Add(LODContainerInstanceNode);
		}
	}

	bool GetPropertyAnimationCurvePayloadData(
		const FString& PayloadKey,
		const UInterchangeUsdContext& UsdContext,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetPropertyAnimationCurvePayloadData)

		FString PrimPath;
		FString UEPropertyNameStr;
		bool bSplit = PayloadKey.Split(TEXT("\\"), &PrimPath, &UEPropertyNameStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!bSplit)
		{
			return false;
		}

		UE::FUsdStage UsdStage = UsdContext.GetUsdStage();
		if (!UsdStage)
		{
			return false;
		}

		UE::FUsdPrim Prim = UsdStage.GetPrimAtPath(UE::FSdfPath{*PrimPath});
		FName UEPropertyName = *UEPropertyNameStr;
		if (!Prim || UEPropertyName == NAME_None)
		{
			return false;
		}

		TArray<double> TimeSampleUnion;
		UsdToUnreal::FPropertyTrackReader Reader;

		TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, UEPropertyName);
		bool bSuccess = UE::FUsdAttribute::GetUnionedTimeSamples(Attrs, TimeSampleUnion);
		if (!bSuccess)
		{
			return false;
		}

		UsdUtils::DensifyTimeSamplesIfNeeded(Prim, UEPropertyName, TimeSampleUnion);

		const bool bIgnorePrimLocalTransform = false;
		Reader = UsdToUnreal::CreatePropertyTrackReader(Prim, UEPropertyName, bIgnorePrimLocalTransform);

		if (Reader.BoolReader)
		{
			return ReadBools(UsdStage, TimeSampleUnion, Reader.BoolReader, OutPayloadData);
		}
		else if (Reader.ColorReader)
		{
			return ReadColors(UsdStage, TimeSampleUnion, Reader.ColorReader, OutPayloadData);
		}
		else if (Reader.FloatReader)
		{
			return ReadFloats(UsdStage, TimeSampleUnion, Reader.FloatReader, OutPayloadData);
		}
		else if (Reader.TransformReader)
		{
			return ReadTransforms(UsdStage, TimeSampleUnion, Reader.TransformReader, OutPayloadData);
		}

		return false;
	}
}	 // namespace UE::GeomImageableSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FImageableSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("ImageableHandler");
		return HandlerName;
	}

	const FString& FImageableSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Imageable");
		return SchemaName;
	}

	TOptional<bool> FImageableSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// For now let's just block the collapsing of anything we recognize as being a multi-LOD mesh
		if (UsdUtils::DoesPrimContainMeshLODs(Prim))
		{
			return false;
		}
		if (Prim.HasAPI(*UsdToUnreal::ConvertToken(UnrealIdentifiers::LodSubtreeAPI)))
		{
			return false;
		}

		return {};
	}

	TOptional<bool> FImageableSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// If we have LOD stuff, let's also prefer to end up as a multi-LOD mesh instead of collapsing our child Mesh prims together
		if (UsdUtils::DoesPrimContainMeshLODs(Prim))
		{
			return false;
		}
		if (Prim.HasAPI(*UsdToUnreal::ConvertToken(UnrealIdentifiers::LodSubtreeAPI)))
		{
			return false;
		}

		// Prevent collapsing of any skinned meshes (our main skel handler is tied to the Skeleton and won't be invoked on the SkelRoot, 
		// so it's good to do this here to try and intercept collapsing all the skinned meshes together)
		if (Prim.IsA(TEXT("SkelRoot")))
		{
			return false;
		}

		return {};
	}

	bool FImageableSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FImageableSchemaHandler::OnTranslate)

		using namespace UE::GeomImageableSchemaHandler::Private;

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);

		// Hide our scene node if it is meant to be hidden in USD
		//
		// We use actor visibility so that it matches how we map visibility timeSamples to actor visibility tracks.
		// We do *that*, because it matches how Interchange always puts transform animations on the actors directly (so
		// "scene component stuff" ends up as actor tracks), and also due to how it behaves better for cameras: Component
		// visibility for camera nodes would hide the camera component itself, which has no effect. Actor visibility for
		// camera actors does hide the entire camera actor however
		if (!TraversalInfo.bVisible)
		{
			if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
			{
				SceneNode->SetCustomActorVisibility(TraversalInfo.bVisible);
			}
			else if (UInterchangeSceneComponentNode* ComponentNode = Cast<UInterchangeSceneComponentNode>(SceneNodeBase))
			{
				ComponentNode->SetCustomComponentVisibility(TraversalInfo.bVisible);
			}
		}

		// Note: We're relying on the UsdGeomImageable handler to provide our animation payload data
		// Binding visibility to the actor works better for cameras
		const static TMap<FString, TArray<FInterchangeTrackInfo>> AttributeMapping = {
			{UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->visibility), {{UnrealIdentifiers::HiddenInGamePropertyName, 	EInterchangePropertyTracks::ActorHiddenInGame, EInterchangeAnimationPayLoadType::STEPCURVE}}},
		};
		AddNodesForAnimatedAttributes(Prim, AttributeMapping, AccumulatedInfo, UsdContext);

		// LOD handling
		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			// Prefer checking for LodSubtree
			if (Prim.HasAPI(*UsdToUnreal::ConvertToken(UnrealIdentifiers::LodSubtreeAPI)))
			{
				HandleLodSubtreeSchemaLODs(Prim, SceneNode, TraversalInfo, AccumulatedInfo, UsdContext);
			}
			// Variant-style LODs
			else if (TraversalInfo.bIsLODVariantContainer)
			{
				HandleVariantStyleLODs(Prim, SceneNode, TraversalInfo, AccumulatedInfo, UsdContext);
			}
		}

		return true;
	}

	bool FImageableSchemaHandler::OnGetAnimationPayloadData(
		const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FImageableSchemaHandler::OnGetAnimationPayloadData)

		using namespace UE::GeomImageableSchemaHandler::Private;

		TArray<int32> CurveQueryIndexes;
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadQueries.Num(); ++PayloadIndex)
		{
			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];

			switch (PayloadQuery.PayloadKey.Type)
			{
				case EInterchangeAnimationPayLoadType::CURVE:	 // Intentional fallthrough
				case EInterchangeAnimationPayLoadType::STEPCURVE:
					CurveQueryIndexes.Add(PayloadIndex);
					break;
				default:
					break;
			}
		}

		int32 CurvePayloadCount = CurveQueryIndexes.Num();
		if (CurvePayloadCount == 0)
		{
			return true;
		}

		TArray<TArray<UE::Interchange::FAnimationPayloadData>> CurveAnimationPayloads;
		auto GetAnimPayloadLambda = [this, &PayloadQueries, &CurveAnimationPayloads, &UsdContext](int32 PayloadIndex)
		{
			if (!PayloadQueries.IsValidIndex(PayloadIndex))
			{
				return;
			}

			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];

			// We're fine handling these in isolation (currently GetAnimationPayloadData is called with
			// a single query at a time for these): Emit a separate task for each right away
			FAnimationPayloadData AnimationPayLoadData{PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey};
			if (GetPropertyAnimationCurvePayloadData(PayloadQuery.PayloadKey.UniqueId, UsdContext, AnimationPayLoadData))
			{
				CurveAnimationPayloads[PayloadIndex].Emplace(AnimationPayLoadData);
			}
		};

		CurveAnimationPayloads.AddDefaulted(CurvePayloadCount);
		const int32 BatchSize = 10;
		if (CurvePayloadCount > BatchSize)
		{
			const int32 NumBatches = (CurvePayloadCount / BatchSize) + 1;
			ParallelFor(
				NumBatches,
				[&CurveQueryIndexes, &GetAnimPayloadLambda](int32 BatchIndex)
				{
					int32 PayloadIndexOffset = BatchIndex * BatchSize;
					for (int32 PayloadIndex = PayloadIndexOffset; PayloadIndex < PayloadIndexOffset + BatchSize; ++PayloadIndex)
					{
						// The last batch can be incomplete
						if (!CurveQueryIndexes.IsValidIndex(PayloadIndex))
						{
							break;
						}
						GetAnimPayloadLambda(CurveQueryIndexes[PayloadIndex]);
					}
				},
				EParallelForFlags::BackgroundPriority
			);
		}
		else
		{
			for (int32 PayloadIndex = 0; PayloadIndex < CurvePayloadCount; ++PayloadIndex)
			{
				int32 PayloadQueriesIndex = CurveQueryIndexes[PayloadIndex];
				if (PayloadQueries.IsValidIndex(PayloadQueriesIndex))
				{
					GetAnimPayloadLambda(PayloadQueriesIndex);
				}
			}
		}

		for (TArray<UE::Interchange::FAnimationPayloadData>& AnimationPayload : CurveAnimationPayloads)
		{
			InOutPayloadData.Append(AnimationPayload);
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
