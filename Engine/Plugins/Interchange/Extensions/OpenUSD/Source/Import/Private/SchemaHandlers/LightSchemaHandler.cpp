// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/LightSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "USDErrorUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDLightConversion.h"
#include "USDLog.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

#include "InterchangeLightNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeTexture2DNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/Scene.h"
#include "InterchangeCommonAnimationPayload.h"
#include "Misc/Paths.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usdLux/tokens.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "LightSchemaHandler"

namespace UE::LuxLightSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	FString AddTextureCubeNode(const FString& TextureFilePath, FHandlerAccumulatedInfo& AccumulatedInfo, UInterchangeBaseNodeContainer& NodeContainer)
	{
		const FString NodeUid = MakeNodeUid(FString::Printf(TEXT("%s%s"), *TexturePrefix, *TextureFilePath));

		// If we created an actual TextureCubeNode here we'd be queried for the sliced payload later. Our HDR image is
		// just a latlong 2d texture though, so we instead need to create a Texture2DNode and then set SetForceLongLatCubemap(),
		// which will cause UInterchangeGenericTexturePipeline::HandleCreationOfTextureFactoryNode() to produce a
		// UInterchangeTextureCubeFactoryNode from this translated node, producing our TextureCube
		UInterchangeTexture2DNode* Node = GetExistingNode<UInterchangeTexture2DNode>(NodeContainer, NodeUid);
		if (!Node)
		{
			const FString NodeName{FPaths::GetBaseFilename(TextureFilePath)};
			Node = NewObject<UInterchangeTexture2DNode>(&NodeContainer);
			NodeContainer.SetupNode(Node, NodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset);
		}

		Node->SetForceLongLatCubemap(true);

		UsdToUnreal::FTextureParameterValue TextureParameter;
		TextureParameter.TextureFilePath = TextureFilePath;
		TextureParameter.Group = TextureGroup::TEXTUREGROUP_Skybox;
		Node->SetPayLoadKey(EncodeTexturePayloadKey(TextureParameter));

		AccumulatedInfo.PrimAssetNodes.Add(Node);

		return NodeUid;
	}

	TSubclassOf<UInterchangeBaseLightNode> GetLightNodeClass(const UE::FUsdPrim& Prim)
	{
		if (Prim.IsA(TEXT("DistantLight")))
		{
			return UInterchangeDirectionalLightNode::StaticClass();
		}
		else if (Prim.IsA(TEXT("SphereLight")))
		{
			if (Prim.HasAPI(TEXT("ShapingAPI")))
			{
				return UInterchangeSpotLightNode::StaticClass();
			}
			else
			{
				return UInterchangePointLightNode::StaticClass();
			}
		}
		else if (Prim.IsA(TEXT("RectLight")) || Prim.IsA(TEXT("DiskLight")))
		{
			return UInterchangeRectLightNode::StaticClass();
		}
		else if (Prim.IsA(TEXT("DomeLight")) || Prim.IsA(TEXT("DomeLight_1")))
		{
			return UInterchangeSkyLightNode::StaticClass();
		}

		return {};
	}

	void ConvertLightPrim(
		const UE::FUsdPrim& Prim,
		UInterchangeBaseLightNode* LightNode,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeBaseNodeContainer& NodeContainer
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertLightPrim)

		if (!Prim || !LightNode)
		{
			return;
		}

		// Ref. UsdToUnreal::ConvertLight
		static const FString IntensityToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsIntensity);
		bool bIntensityAuthored = false;
		float IntensityUsd = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*IntensityToken), UsdUtils::GetDefaultTimeCode(), &bIntensityAuthored);

		bool bExposureAuthored = false;
		static const FString ExposureToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsExposure);
		float Exposure = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*ExposureToken), UsdUtils::GetDefaultTimeCode(), &bExposureAuthored);

		static const FString NormalizeToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsNormalize);
		bool bNormalizeAuthored = false;
		bool bNormalizedIntensityUsd = UsdUtils::GetLightAttrValueWithInputsFallback<bool>(Prim.GetAttribute(*NormalizeToken), UsdUtils::GetDefaultTimeCode(), &bNormalizeAuthored);
		bNormalizedIntensityUsd = bNormalizeAuthored && bNormalizedIntensityUsd;
		if (bNormalizeAuthored)
		{
			USD_LOG_WARNING(TEXT("Light prim %s has 'inputs:normalize' authored, which is not supported!"), *Prim.GetPrimPath().GetString());
			LightNode->AddBooleanAttribute(LightAPINormalizeAttributeKey, bNormalizedIntensityUsd);
		}

		static const FString RadiusToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsRadius);

		// Temporarily a bit awkward here as we updated our VtValue::Get() to not perform any type coercion, so we
		// need to do that ourselves. In the future we'll have a dedicated templated function to perform this process
		FLinearColor Color = FLinearColor::White;
		static const FString ColorToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsColor);
		UE::FUsdAttribute ColorAttribute = Prim.GetAttribute(*ColorToken);
		if (!ColorAttribute || !ColorAttribute.HasAuthoredValue())
		{
			// Fallback to the unprefixed attribute for pre-21.05 USD scenes
			static const FString UnprefixedColorToken = TEXT("color");
			UE::FUsdAttribute FallbackAttr = Prim.GetAttribute(*UnprefixedColorToken);
			if (FallbackAttr)
			{
				ColorAttribute = FallbackAttr;
			}
		}
		if (ColorAttribute && ColorAttribute.HasAuthoredValue())
		{
			UE::FVtValue Value;
			if (ColorAttribute.Get(Value))
			{
				const FString TypeName = Value.GetTypeName();
				if (TypeName == TEXT("GfVec3f"))
				{
					Color = FLinearColor{Value.Get<FVector3f>()};
				}
				else if (TypeName == TEXT("GfVec3h"))
				{
					Color = FLinearColor{Value.Get<FVector3h>()};
				}
				else if (TypeName == TEXT("GfVec3d"))
				{
					Color = FLinearColor{Value.Get<FVector3d>()};
				}
				if (TypeName == TEXT("GfVec4f"))
				{
					Color = FLinearColor{Value.Get<FVector4f>()};
				}
				else if (TypeName == TEXT("GfVec4h"))
				{
					Color = FLinearColor{Value.Get<FVector4h>()};
				}
				else if (TypeName == TEXT("GfVec4d"))
				{
					Color = FLinearColor{Value.Get<FVector4d>()};
				}
			}

			const bool bSRGB = true;
			Color = Color.ToFColor(bSRGB);
			LightNode->SetCustomLightColor(Color);
		}

		static const FString TemperatureToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsColorTemperature);
		bool bTemperatureAuthored = false;
		float Temperature = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*TemperatureToken), UsdUtils::GetDefaultTimeCode(), &bTemperatureAuthored); 
		if (bTemperatureAuthored)
		{
			LightNode->SetCustomTemperature(Temperature);
		}

		static const FString UseTemperatureToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsEnableColorTemperature);
		bool bUseTemperatureAuthored = false;
		bool UseTemperature = UsdUtils::GetLightAttrValueWithInputsFallback<bool>(Prim.GetAttribute(*UseTemperatureToken), UsdUtils::GetDefaultTimeCode(), &bUseTemperatureAuthored); 
		if (bUseTemperatureAuthored)
		{
			LightNode->SetCustomUseTemperature(UseTemperature);
		}

		float SolidAngle = 1.0f;
		float AreaSqMeters = 1.0f;
		if (Prim.IsA(TEXT("DistantLight")))
		{
			UInterchangeDirectionalLightNode* CastLightNode = Cast<UInterchangeDirectionalLightNode>(LightNode);
			if (!ensure(CastLightNode))
			{
				return;
			}

			static const FString SourceAngleToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsAngle);
			bool bAngleAuthored = false;
			float UsdAngle = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*SourceAngleToken), UsdUtils::GetDefaultTimeCode(), &bAngleAuthored); 
			if (bAngleAuthored)
			{
				CastLightNode->SetCustomLightSourceAngle(UsdAngle);
			}
		}
		else if (Prim.IsA(TEXT("SphereLight")))
		{
			const FUsdStageInfo StageInfo(Prim.GetStage());

			float RadiusUE = 1.0f;
			if (UInterchangePointLightNode* CastLightNode = Cast<UInterchangePointLightNode>(LightNode))
			{
				bool bRadiusAuthored = false;
				float RadiusUsd = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*RadiusToken), UsdUtils::GetDefaultTimeCode(), &bRadiusAuthored); 
				if (bRadiusAuthored)
				{
					RadiusUE = UsdToUnreal::ConvertDistance(StageInfo, RadiusUsd);
					CastLightNode->SetCustomSourceRadius(RadiusUE);
				}
			}

			if (Prim.HasAPI(TEXT("ShapingAPI")))
			{
				UInterchangeSpotLightNode* CastLightNode = Cast<UInterchangeSpotLightNode>(LightNode);
				if (!ensure(CastLightNode))
				{
					return;
				}

				static const FString ConeAngleToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsShapingConeAngle);
				static const FString ConeSoftnessToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsShapingConeSoftness);

				// Default from UsdLuxShapingAPI
				bool bConeAngleAuthored = false;
				float UsdConeAngle = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*ConeAngleToken), UsdUtils::GetDefaultTimeCode(), &bConeAngleAuthored);

				// Default from UsdLuxShapingAPI
				bool bConeSoftnessAuthored = false;
				float UsdConeSoftness = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*ConeSoftnessToken), UsdUtils::GetDefaultTimeCode(), &bConeSoftnessAuthored);

				float InnerConeAngle = 0.0f;
				float OuterConeAngle = 44.0f; // Default from the USpotLightComponent constructor
				UsdToUnreal::ConvertSpotLightConeAngles(UsdConeAngle, UsdConeSoftness, OuterConeAngle, InnerConeAngle);

				SolidAngle = UsdUtils::GetSpotLightSolidAngle(OuterConeAngle, InnerConeAngle);
				AreaSqMeters = UsdUtils::GetSpotLightArea(RadiusUE / 100.0f);

				if (bConeAngleAuthored || bConeSoftnessAuthored)
				{
					CastLightNode->SetCustomInnerConeAngle(InnerConeAngle);
					CastLightNode->SetCustomOuterConeAngle(OuterConeAngle);
				}
			}
			else
			{
				SolidAngle = UsdUtils::GetPointLightSolidAngle();
				AreaSqMeters = UsdUtils::GetSpotLightArea(RadiusUE / 100.0f);
			}
		}
		else if (Prim.IsA(TEXT("RectLight")) || Prim.IsA(TEXT("DiskLight")))
		{
			UInterchangeRectLightNode* CastLightNode = Cast<UInterchangeRectLightNode>(LightNode);
			if (!ensure(CastLightNode))
			{
				return;
			}

			static const FString WidthToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsWidth);
			static const FString HeightToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsHeight);

			const FUsdStageInfo StageInfo(Prim.GetStage());

			if (Prim.IsA(TEXT("RectLight")))
			{
				float WidthUE = 1.0f;
				bool bWidthAuthored = false;
				float WidthUsd = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*WidthToken), UsdUtils::GetDefaultTimeCode(), &bWidthAuthored);
				if (bWidthAuthored)
				{
					WidthUE = UsdToUnreal::ConvertDistance(StageInfo, WidthUsd);
					CastLightNode->SetCustomSourceWidth(WidthUE);
				}

				float HeightUE = 1.0f;
				bool bHeightAuthored = false;
				float HeightUsd = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*HeightToken), UsdUtils::GetDefaultTimeCode(), &bHeightAuthored); 
				if (bHeightAuthored)
				{
					HeightUE = UsdToUnreal::ConvertDistance(StageInfo, HeightUsd);
					CastLightNode->SetCustomSourceHeight(HeightUE);
				}

				SolidAngle = UsdUtils::GetRectLightSolidAngle();
				AreaSqMeters = UsdUtils::GetRectLightArea(WidthUE / 100.0f, HeightUE / 100.0f);
			}
			else // DiskLight
			{
				float RadiusUE = 0.5f;
				bool bDiskRadiusAuthored = false;
				float RadiusUsd = UsdUtils::GetLightAttrValueWithInputsFallback<float>(Prim.GetAttribute(*RadiusToken), UsdUtils::GetDefaultTimeCode(), &bDiskRadiusAuthored); 
				if (bDiskRadiusAuthored)
				{
					RadiusUE = UsdToUnreal::ConvertDistance(StageInfo, RadiusUsd);
					float SquareSideUE = UsdUtils::GetRectLightSideFromDiskLight(RadiusUE);
					CastLightNode->SetCustomSourceWidth(SquareSideUE);
					CastLightNode->SetCustomSourceHeight(SquareSideUE);
				}

				SolidAngle = UsdUtils::GetDiskLightSolidAngle();
				AreaSqMeters = UsdUtils::GetDiskLightArea(RadiusUE / 100.0f);
			}
		}
		else if (Prim.IsA(TEXT("DomeLight")) || Prim.IsA(TEXT("DomeLight_1")))
		{
			UInterchangeSkyLightNode* CastLightNode = Cast<UInterchangeSkyLightNode>(LightNode);
			if (!ensure(CastLightNode))
			{
				return;
			}

			// Show some warning in case the file specifies a different DomeLight format, as we only support latlong
			static const FString FormatToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsTextureFormat);
			UE::FUsdAttribute FormatAttribute = Prim.GetAttribute(*FormatToken);
			if (!FormatAttribute || !FormatAttribute.HasAuthoredValue())
			{
				static const FString UnprefixedFormatToken = TEXT("textureFormat");
				UE::FUsdAttribute FallbackAttr = Prim.GetAttribute(*UnprefixedFormatToken);
				if (FallbackAttr)
				{
					FormatAttribute = FallbackAttr;
				}
			}
			FString Format;
			if (FormatAttribute.HasAuthoredValue() &&
				FormatAttribute.Get<FString>(Format) &&
				!Format.IsEmpty() &&
				Format != TEXT("automatic") &&
				Format != TEXT("latlong"))
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"UnsupportedFormat",
						"DomeLight '{0}' specifies format '{1}', but only 'latlong' DomeLights are supported for now. The texture will be treated as 'latlong'"
					),
					FText::FromString(Prim.GetPrimPath().GetString()),
					FText::FromString(Format)
				));
			}

			static const FString FileToken = UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsTextureFile);
			UE::FUsdAttribute FileAttribute = Prim.GetAttribute(*FileToken);
			if (!FileAttribute || !FileAttribute.HasAuthoredValue())
			{
				static const FString UnprefixedFileToken = TEXT("textureFile");
				UE::FUsdAttribute FallbackAttr = Prim.GetAttribute(*UnprefixedFileToken);
				if (FallbackAttr)
				{
					FileAttribute = FallbackAttr;
				}
			}
			const FString ResolvedDomeTexturePath = UsdUtils::GetResolvedAssetPath(FileAttribute);

			FString TextureCubeNodeUid = AddTextureCubeNode(ResolvedDomeTexturePath, AccumulatedInfo, NodeContainer);
			if (!TextureCubeNodeUid.IsEmpty())
			{
				CastLightNode->SetCustomCubemapDependency(TextureCubeNodeUid);
				CastLightNode->SetCustomSourceType(EInterchangeSkyLightSourceType::SpecifiedCubemap);
			}

			// If this file doesn't exist then retrieving the payload will fail, but we emit the warning from here as
			// we can provide a better error message than the the payload retrieval function
			if (!FPaths::FileExists(ResolvedDomeTexturePath))
			{
				// Prefer to retrieve the original filepath string as this ResolvedDomeTexturePath will likely just
				// be the empty string if we failed to resolve
				FString RawFilePath = ResolvedDomeTexturePath;
				if (FileAttribute)
				{
					FileAttribute.Get(RawFilePath);
				}

				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("MissingLightTexture", "Failed to find the file '{0}' specified by DomeLight prim '{1}'"),
					FText::FromString(RawFilePath),
					FText::FromString(Prim.GetPrimPath().GetString())
				));
			}
		}

		if (bIntensityAuthored || bExposureAuthored)
		{
			// This will end up being Nits for local lights (point, rect, disk, spot), lux for distant lights,
			// and a simple multiplier without units for dome lights
			const float Intensity = UsdToUnreal::ConvertLightIntensity(IntensityUsd, Exposure);

			LightNode->SetCustomIntensity(Intensity);
			if (UInterchangeLightNode* DerivedLightNode = Cast<UInterchangeLightNode>(LightNode))
			{
				// Note that directional and sky lights are not LightNodes (they derive BaseLightNode directly), so this
				// check prevents us from trying to set them with Nits
				DerivedLightNode->SetCustomIntensityUnits(EInterchangeLightUnits::Nits);
			}
		}
	}
}	 // namespace UE::LuxLightSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FLightSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("LightHandler");
		return HandlerName;
	}

	const FString& FLightSchemaHandler::GetTargetSchemaName() const
	{
		// A bit awkward here, but this is in fact the most derived class that both light base classes have in common...
		const static FString SchemaName = TEXT("Xformable");
		return SchemaName;
	}

	bool FLightSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLightSchemaHandler::CanHandlePrim)

		return (Prim.IsA(TEXT("BoundableLightBase")) || Prim.IsA(TEXT("NonboundableLightBase"))) && Prim.HasAPI(TEXT("LightAPI"));
	}

	TOptional<bool> FLightSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FLightSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FLightSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLightSchemaHandler::OnTranslate)

		using namespace UE::LuxLightSchemaHandler::Private;

		TSubclassOf<UInterchangeBaseLightNode> LightNodeClass = GetLightNodeClass(Prim);
		if (!LightNodeClass)
		{
			return false;
		}

		FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, LightPrefix);
		FString NewNodeName{Prim.GetName().ToString()};

		UInterchangeBaseNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode(
			LightNodeClass,
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, Prim.GetPrimPath().GetString());

		UInterchangeBaseLightNode* LightAssetNode = Cast<UInterchangeBaseLightNode>(AssetNode);
		if (!ensure(LightAssetNode))
		{
			return false;
		}

		ConvertLightPrim(Prim, LightAssetNode, AccumulatedInfo, *UsdContext.GetNodeContainer());

		// Add nodes for animated attributes
		const TMap<FString, TArray<FInterchangeTrackInfo>>* AttributeMapping = nullptr;
		{
			const static TMap<FString, TArray<FInterchangeTrackInfo>> CommonAttributeMapping = {
				{UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsIntensity), 				{{UnrealIdentifiers::IntensityPropertyName, 		EInterchangePropertyTracks::LightIntensity}}},
				{UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsExposure), 					{{UnrealIdentifiers::IntensityPropertyName, 		EInterchangePropertyTracks::LightIntensity}}},
				{UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsColor), 					{{UnrealIdentifiers::LightColorPropertyName, 		EInterchangePropertyTracks::LightColor}}},
				{UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsColorTemperature), 			{{UnrealIdentifiers::TemperaturePropertyName, 		EInterchangePropertyTracks::LightTemperature}}},
				{UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsEnableColorTemperature), 	{{UnrealIdentifiers::UseTemperaturePropertyName, 	EInterchangePropertyTracks::LightUseTemperature}}},
			};
	
			// Per-light-type mapping
			// Our main concern here is to capture the properties that make sense for this asset node. If we provide
			// incorrect ones, the main consequence will be broken bindings on the LevelSequence and maybe some errors/warnings
			if (AssetNode->IsA<UInterchangeSpotLightNode>())
			{
				const static TMap<FString, TArray<FInterchangeTrackInfo>> CombinedLightMapping = []()
				{
					TMap<FString, TArray<FInterchangeTrackInfo>> Combined = CommonAttributeMapping;
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsShapingConeAngle), 		{{UnrealIdentifiers::OuterConeAnglePropertyName, 	EInterchangePropertyTracks::LightOuterConeAngle}});
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsShapingConeSoftness), 	{{UnrealIdentifiers::InnerConeAnglePropertyName, 	EInterchangePropertyTracks::LightInnerConeAngle}});
					return Combined;
				}();
				AttributeMapping = &CombinedLightMapping;
			}
			else if (AssetNode->IsA<UInterchangePointLightNode>())
			{
				const static TMap<FString, TArray<FInterchangeTrackInfo>> CombinedLightMapping = []()
				{
					TMap<FString, TArray<FInterchangeTrackInfo>> Combined = CommonAttributeMapping;
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsRadius), 	{{UnrealIdentifiers::SourceRadiusPropertyName, 		EInterchangePropertyTracks::LightSourceRadius}});
					return Combined;
				}();
				AttributeMapping = &CombinedLightMapping;
			}
			else if (AssetNode->IsA<UInterchangeRectLightNode>())
			{
				const static TMap<FString, TArray<FInterchangeTrackInfo>> CombinedLightMapping = []()
				{
					TMap<FString, TArray<FInterchangeTrackInfo>> Combined = CommonAttributeMapping;
					// Animated inputsRadius on a RectLightNode means this is a DiskLight handled as a RectLight
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsRadius), 	{{UnrealIdentifiers::SourceWidthPropertyName, 		EInterchangePropertyTracks::LightSourceWidth},
																								 {UnrealIdentifiers::SourceHeightPropertyName, 		EInterchangePropertyTracks::LightSourceHeight}});
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsWidth), 	{{UnrealIdentifiers::SourceWidthPropertyName, 		EInterchangePropertyTracks::LightSourceWidth}});
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsHeight), 	{{UnrealIdentifiers::SourceHeightPropertyName, 		EInterchangePropertyTracks::LightSourceHeight}});
					return Combined;
				}();
				AttributeMapping = &CombinedLightMapping;
			}
			else if (AssetNode->IsA<UInterchangeDirectionalLightNode>())
			{
				const static TMap<FString, TArray<FInterchangeTrackInfo>> CombinedLightMapping = []()
				{
					TMap<FString, TArray<FInterchangeTrackInfo>> Combined = CommonAttributeMapping;
					Combined.Add(UsdToUnreal::ConvertToken(pxr::UsdLuxTokens->inputsAngle), 	{{UnrealIdentifiers::LightSourceAnglePropertyName, 	EInterchangePropertyTracks::LightSourceAngle}});
					return Combined;
				}();
				AttributeMapping = &CombinedLightMapping;
			}
			else // UInterchangeSkyLightNode
			{
				AttributeMapping = &CommonAttributeMapping;
			}
		}
		if (AttributeMapping != nullptr)
		{
			AddNodesForAnimatedAttributes(Prim, *AttributeMapping, AccumulatedInfo, UsdContext);
		}

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			SceneNode->SetCustomAssetInstanceUid(AssetNode->GetUniqueID());
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
