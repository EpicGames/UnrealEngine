// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/MaterialSchemaHandler.h"

#include "USDShadeConversion.h"
#include "UsdWrappers/UsdPrim.h"

namespace UE::Interchange::USD
{
	const FString& FMaterialSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Material");
		return SchemaName;
	}

	bool FMaterialSchemaHandler::AllowCustomRenderContexts() const
	{
		return false;
	}

	const TArray<FString>& FMaterialSchemaHandler::GetCustomRenderContexts() const
	{
		return CustomRenderContexts;
	}

	void FMaterialSchemaHandler::SetCustomRenderContexts(const TArray<FString>& RenderContexts)
	{
		if (!ensure(AllowCustomRenderContexts()))
		{
			return;
		}

		CustomRenderContexts = RenderContexts;
	}

	bool FMaterialSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialSchemaHandler::CanHandlePrim)

		if (!Prim.IsA(*GetTargetSchemaName()))
		{
			return false;
		}

		return UsdUtils::HasSurfaceOutput(Prim, GetDefaultRenderContexts());
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
