// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/MaterialXSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "USDShadeConversion.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "MaterialX/InterchangeMaterialXTranslator.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "USDMaterialXShaderGraph.h"

#include "InterchangeManager.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMaterialReferenceNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTextureNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Misc/Paths.h"

namespace UE::MaterialXSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	// Returns the UID of the material translated node that was generated from the MaterialX translation of a particular
	// material prim.
	//
	// This works because when parsing MaterialX files we generate shader graph nodes with UIDs that match the original
	// material prim name in the USD file (e.g. on USD we have a binding relationship to </MaterialX/Materials/Marble_3D>,
	// and we end up generating a shader graph node with Uid \\Shaders\\Marble_3D).
	FString GetMaterialXMaterialUid(const FString& PrimName, UInterchangeBaseNodeContainer& NodeContainer)
	{
		FString Result;

		NodeContainer.BreakableIterateNodesOfType<UInterchangeShaderGraphNode>(
			[&NodeContainer, &PrimName, &Result](const FString&, UInterchangeShaderGraphNode* ShaderGraphNode)
			{
				FString NodeUid = ShaderGraphNode->GetUniqueID();
				// Because USD allows having the same name between a parent and its child (unlike MaterialX)
				// When building the shader graph we may end up with a name not totally equal to the USD one (MaterialX will append an index to the material name)
				// e.g: prim -> mtlx_volume
				//      mtlx -> mtlx_volume2
				// see the egg_jade file from https://github.com/stehrani3d/MaterialEggs
				FString BaseName = FPaths::GetBaseFilename(NodeUid);
				if (BaseName == PrimName)
				{
					Result = NodeUid;
					return true;
				}
				else
				{
					if (BaseName.StartsWith(PrimName, ESearchCase::CaseSensitive))
					{
						if (FString Suffix = BaseName.Mid(PrimName.Len()); Suffix.IsNumeric())
						{
							Result = NodeUid;
							return true;
						}
					}
					return false;
				}
			}
		);

		return Result;
	}

	// Walks the shader graph rooted at RootMaterial via UInterchangeShaderPortsAPI, finds every
	// UInterchangeTextureNode reachable through input connections and string-valued inputs, and tags
	// each of them with PrimPathString via UE::Interchange::USD::SetPrimPath. Intermediate
	// UInterchangeShaderNodes (e.g. TextureSample, math nodes) are followed but not tagged themselves,
	// since they don't produce UAssets by themselves: only the root material and the texture nodes do,
	// and those are the only ones the pregen pipeline ever needs to map back to a USD prim path.
	//
	// Note that some texture references (notably the MaterialX TextureSample pattern) live as Value
	// or Parameter string attributes on a shader node's input rather than as a connection (see
	// MaterialXSurfaceShaderAbstract.cpp ~line 605, which writes the texture UID via
	// MakeInputValueKey(...)), so we check both connections and string-valued inputs at every node.
	void TagTexturesReachableFromMaterial(
		UInterchangeShaderNode& RootMaterial,
		UInterchangeBaseNodeContainer& NodeContainer,
		const FString& PrimPathString
	)
	{
		TSet<FString> Visited;
		TArray<UInterchangeShaderNode*> ToProcess;

		Visited.Add(RootMaterial.GetUniqueID());
		ToProcess.Push(&RootMaterial);

		while (!ToProcess.IsEmpty())
		{
			UInterchangeShaderNode* Current = ToProcess.Pop();

			TArray<FString> InputNames;
			UInterchangeShaderPortsAPI::GatherInputs(Current, InputNames);

			for (const FString& InputName : InputNames)
			{
				// 1) Follow shader-graph connections. This is how the material's BaseColor (etc.) reaches
				// a TextureSample shader node, and how chained math nodes link together.
				FString ConnectedUid;
				FString OutputName;
				if (UInterchangeShaderPortsAPI::GetInputConnection(Current, InputName, ConnectedUid, OutputName)
					&& !ConnectedUid.IsEmpty())
				{
					bool bAlreadyVisited = false;
					Visited.Add(ConnectedUid, &bAlreadyVisited);
					if (!bAlreadyVisited)
					{
						if (UInterchangeBaseNode* Connected = const_cast<UInterchangeBaseNode*>(NodeContainer.GetNode(ConnectedUid)))
						{
							if (UInterchangeTextureNode* TextureNode = Cast<UInterchangeTextureNode>(Connected))
							{
								UE::Interchange::USD::SetPrimPath(*TextureNode, PrimPathString);
							}
							else if (UInterchangeShaderNode* ShaderNode = Cast<UInterchangeShaderNode>(Connected))
							{
								ToProcess.Push(ShaderNode);
							}
						}
					}
				}

				// 2) Check string-valued inputs. MaterialX TextureSample stores the texture's UID as a
				// string attribute under Inputs:Texture:Value (or :Parameter), not as a connection -
				// so a pure connection walk would miss every MaterialX-bound texture without this step.
				const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(Current, FName{*InputName});
				const FString InputKey = bIsAParameter
					? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName)
					: UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);

				FString CandidateUid;
				if (Current->GetStringAttribute(InputKey, CandidateUid) && !CandidateUid.IsEmpty())
				{
					bool bAlreadyVisited = false;
					Visited.Add(CandidateUid, &bAlreadyVisited);
					if (!bAlreadyVisited)
					{
						if (UInterchangeBaseNode* Referenced = const_cast<UInterchangeBaseNode*>(NodeContainer.GetNode(CandidateUid)))
						{
							if (UInterchangeTextureNode* TextureNode = Cast<UInterchangeTextureNode>(Referenced))
							{
								UE::Interchange::USD::SetPrimPath(*TextureNode, PrimPathString);
							}
						}
					}
				}
			}
		}
	}

	bool AddMaterialXShaderGraph(const UE::FUsdPrim& Prim, TArray<FUsdMaterialXShaderGraph::FGeomProp>& GeomPropValues, UInterchangeUsdContext& UsdContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddMaterialXShaderGraph)

#if WITH_EDITOR
		TArray<FString> FilePaths = UsdUtils::GetMaterialXFilePaths(Prim);
		for (const FString& File : FilePaths)
		{
			// the file has already been handled no need to do a Translate again
			if (!UsdContext.SubTranslators.Find(File))
			{
				UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
				UInterchangeSourceData* SourceData = UInterchangeManager::CreateSourceData(File);

				UInterchangeTranslatorBase* Translator = InterchangeManager.GetTranslatorForSourceData(SourceData);
				// check on the Translator, it might return nullptr in case of reimport
				if (Translator)
				{
					Translator->Results = UsdContext.GetResultsContainer();
					Translator->Translate(*UsdContext.GetNodeContainer());
					UsdContext.SubTranslators.Emplace(File, Translator);
				}
			}
		}

		if (!FilePaths.IsEmpty())
		{
			return true;
		}

		// Conditionally enable the shader graph for the time being as it crashes on Linux cause of a probable double free
#if ENABLE_USD_MATERIALX
		FUsdMaterialXShaderGraph ShaderGraph(Prim, *UnrealIdentifiers::MaterialXRenderContext.ToString());
		GeomPropValues = ShaderGraph.GetGeomPropValueNames();
		if (MaterialX::DocumentPtr Document = ShaderGraph.GetDocument())
		{
			return FMaterialXManager::GetInstance().Translate(Document, *UsdContext.GetNodeContainer());
		}
#endif // ENABLE_USD_MATERIALX

#endif	  // WITH_EDITOR
		return false;
	}

}	 // namespace UE::MaterialXSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FMaterialXSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("MaterialXHandler");
		return HandlerName;
	}

	const TArray<FString>& FMaterialXSchemaHandler::GetDefaultRenderContexts() const
	{
		const static TArray<FString> RenderContexts{UnrealIdentifiers::MaterialXRenderContext.ToString()};
		return RenderContexts;
	}

	bool FMaterialXSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialXSchemaHandler::CanHandlePrim)

#if WITH_EDITOR && ENABLE_USD_MATERIALX
		if (!Prim.IsA(*GetTargetSchemaName()))
		{
			return false;
		}

		const TArray<FString>& RenderContexts = GetDefaultRenderContexts();
		return UsdUtils::HasSurfaceOutput(Prim, RenderContexts)
			   || UsdUtils::HasVolumeOutput(Prim, RenderContexts)
			   || UsdUtils::HasDisplacementOutput(Prim, RenderContexts);
#else	 // #if WITH_EDITOR && ENABLE_USD_MATERIALX
		return false;
#endif
	}

	bool FMaterialXSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialXSchemaHandler::OnTranslate)

		using namespace UE::MaterialXSchemaHandler::Private;

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return {};
		}

		// For the material schema handlers, we only touch the asset if no other handler has produced a material node yet.
		// This helps manage the separate material handlers per render context, without having them partially overwrite the
		// node with render-context-specific data over and over
		const static TSet<UClass*> MaterialLikeClasses = {
			UInterchangeMaterialInstanceNode::StaticClass(),
			UInterchangeShaderGraphNode::StaticClass(),
			UInterchangeMaterialReferenceNode::StaticClass()
		};
		for (const UInterchangeBaseNode* AssetNode : AccumulatedInfo.PrimAssetNodes)
		{
			for (const UClass* MaterialClass : MaterialLikeClasses)
			{
				if (AssetNode->IsA(MaterialClass))
				{
					return false;
				}
			}
		}

		FString MaterialPrimName = Prim.GetName().ToString();

		TArray<FUsdMaterialXShaderGraph::FGeomProp> GeomPropValues;
		const bool bParsedMaterialX = AddMaterialXShaderGraph(Prim, GeomPropValues, UsdContext);
		if (!bParsedMaterialX)
		{
			return false;
		}

		// AddMaterialXShaderGraph should have already created the Material nodes we need, we just need to find their UIDs
		// TODO: Need better detection and collection of *all* nodes produced by a subtranslator like this
		FString MaterialXMaterialUid = GetMaterialXMaterialUid(MaterialPrimName, *NodeContainer);

		// Add the main shadergraph material node to the list of assets produced for this prim
		if (UInterchangeShaderGraphNode* MaterialXMaterial = const_cast<UInterchangeShaderGraphNode*>(Cast<UInterchangeShaderGraphNode>(NodeContainer->GetNode(MaterialXMaterialUid))))
		{
			// Insert the main material at the front so we run into it first, as in case we have twosided materials we want to use the
			// twosided version
			const uint32 Index = 0;
			AccumulatedInfo.PrimAssetNodes.Insert(MaterialXMaterial, Index);

			// Annotate shader graph nodes and texture nodes with the prim path (needed for pregen)
			const FString PrimPathString = Prim.GetPrimPath().GetString();
			UE::Interchange::USD::SetPrimPath(*MaterialXMaterial, PrimPathString);
			TagTexturesReachableFromMaterial(*MaterialXMaterial, *NodeContainer, PrimPathString);
		}

		if (GeomPropValues.Num() > 0)
		{
			UsdContext.MaterialUidToGeomProps.Add(MaterialXMaterialUid, GeomPropValues);
		}

		return true;
	}

	bool FMaterialXSchemaHandler::OnGetTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportImage>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialXSchemaHandler::OnGetTexturePayloadData)

		// We did not find a suitable Payload in USD Translator, let's find one in one of the Translators (MaterialX for the moment)
		// The best way would be to have a direct association between the payload and the right Translator, but we don't have a suitable way of
		// knowing which Payload belongs to which Translator So let's just loop over them all
		if (!InOutPayloadData.IsSet())
		{
			for (const TPair<FString, TStrongObjectPtr<UInterchangeTranslatorBase>>& Pair : UsdContext.SubTranslators)
			{
				if (IInterchangeTexturePayloadInterface* TexturePayloadInterface = Cast<IInterchangeTexturePayloadInterface>(Pair.Value.Get()))
				{
					InOutPayloadData = TexturePayloadInterface->GetTexturePayloadData(PayloadKey, AlternateTexturePath);
					if (InOutPayloadData.IsSet())
					{
						break;
					}
				}
			}
		}

		// If we couldn't find a texture in either the USD Translator nor the Translators, then it's most likely coming from reading an mtlx in memory
		if (!InOutPayloadData.IsSet())
		{
			InOutPayloadData = UInterchangeMaterialXTranslator::GetTexturePayloadData(
				PayloadKey,
				AlternateTexturePath,
				UsdContext.GetResultsContainer(),
				UsdContext.GetTranslator()->AnalyticsHandler
			);
		}

		return InOutPayloadData.IsSet();
	}

	bool FMaterialXSchemaHandler::OnGetBlockedTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportBlockedImage>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialXSchemaHandler::OnGetBlockedTexturePayloadData)

		// We did not find a suitable Payload in USD Translator, let's find one in one of the Translators (MaterialX for the moment)
		// The best way would be to have a direct association between the payload and the right Translator, but we don't have a suitable way of
		// knowing which Payload belongs to which Translator So let's just loop over them all
		for (const TPair<FString, TStrongObjectPtr<UInterchangeTranslatorBase>>& Pair : UsdContext.SubTranslators)
		{
			if (IInterchangeBlockedTexturePayloadInterface* TexturePayloadInterface = Cast<IInterchangeBlockedTexturePayloadInterface>(Pair.Value.Get()))
			{
				InOutPayloadData = TexturePayloadInterface->GetBlockedTexturePayloadData(PayloadKey, AlternateTexturePath);
				if (InOutPayloadData.IsSet())
				{
					break;
				}
			}
		}

		// If we couldn't find a texture in either the USD Translator nor the Translators, then it's most likely coming from reading an mtlx in
		// memory
		if (!InOutPayloadData.IsSet())
		{
			InOutPayloadData = UInterchangeMaterialXTranslator::GetBlockedTexturePayloadData(
				PayloadKey,
				AlternateTexturePath,
				UsdContext.GetResultsContainer(),
				UsdContext.GetTranslator()->AnalyticsHandler
			);
		}

		return InOutPayloadData.IsSet();
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
