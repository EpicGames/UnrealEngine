// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/PointInstancerSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDErrorUtils.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdEditContext.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdTyped.h"
#include "UsdWrappers/UsdVariantSets.h"

#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "PointInstancerSchemaHandler"

namespace UE::GeomPointInstancerSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	//Helper to traverse PointInstancer subhierarchy and process it into UInterchangeSceneComponents and InstancedStaticMeshComponent.
	struct FISMComponentTraversalHelper
	{
		UInterchangeUsdContext& UsdContext;
		FHandlerAccumulatedInfo& AccumulatedInfo;
		UInterchangeSceneNode* ParentSceneNode;
		const FString ParentSceneNodeUid;

		// While we could lookup from the NodeContainer,
		// we are implementing a LookUp table specifically for ISMComponentNodes for speed:
		TMap<FString, UInterchangeInstancedStaticMeshComponentNode*> UidToComponentNode;

		FISMComponentTraversalHelper(UInterchangeUsdContext& InUsdContext, FHandlerAccumulatedInfo& InAccumulatedInfo, UInterchangeSceneNode* InParentSceneNode)
			: UsdContext(InUsdContext)
			, AccumulatedInfo(InAccumulatedInfo)
			, ParentSceneNode(InParentSceneNode)
			, ParentSceneNodeUid(InParentSceneNode->GetUniqueID())
		{
		}

		UInterchangeMeshNode* TranslateMeshNode(const UE::FUsdPrim& MeshPrim, FTraversalInfo& Info)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TranslateMeshNode)

			if (UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator())
			{
				// We need this here because we're parsing a component hierarchy, and don't want the handlers to produce new scene (i.e. actor) nodes
				const bool bAllowSceneNodeGeneration = false;
				TOptional<FHandlerAccumulatedInfo> MeshPrimAccumulatedInfo = CurrentTranslator->TranslatePrim(MeshPrim, Info, bAllowSceneNodeGeneration);
				if (MeshPrimAccumulatedInfo)
				{
					AccumulatedInfo.AppendInfo(MeshPrimAccumulatedInfo.GetValue());
					return MeshPrimAccumulatedInfo->GetAssetNodeOfClass<UInterchangeMeshNode>();
				}
			}

			return nullptr;
		}

		UInterchangeInstancedStaticMeshComponentNode* CreateSceneComponent(const FString& MeshUid, UE::FUsdPrim Prim)
		{
			FString UniqueId = MakeNodeUid(ComponentPrefix + TEXT("\\") + ParentSceneNodeUid + TEXT("\\") + MeshUid);

			UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = UidToComponentNode.FindRef(UniqueId);
			if (ISMComponentNode)
			{
				return ISMComponentNode;
			}

			UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
			if (!NodeContainer)
			{
				return nullptr;
			}

			const UInterchangeBaseNode* MeshNodeOrLODContainer = NodeContainer->GetNode(MeshUid);
			if (!MeshNodeOrLODContainer)
			{
				return nullptr;
			}

			ISMComponentNode = NewObject<UInterchangeInstancedStaticMeshComponentNode>(NodeContainer);
			FString DisplayLabel = MeshNodeOrLODContainer->GetDisplayLabel();

			NodeContainer->SetupNode(ISMComponentNode, UniqueId, DisplayLabel, EInterchangeNodeContainerType::TranslatedScene, ParentSceneNode->GetUniqueID());
			ParentSceneNode->AddComponentUid(ISMComponentNode->GetUniqueID());

			UidToComponentNode.Add(UniqueId, ISMComponentNode);
			AccumulatedInfo.PrimSceneNodes.Add(ISMComponentNode);

			return ISMComponentNode;
		}

		// TODO: Extract this code, as the Imageable handler also does 90% the exact same thing
		FString ConvertLodContainerPrim(const UE::FUsdPrim& Prim, FString& ActiveVariantUSDPrimPath, FTraversalInfo& Info)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertLodContainerPrim)

			UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator();
			if (!CurrentTranslator)
			{
				return {};
			}

			UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
			if (!NodeContainer)
			{
				return {};
			}

			FString LODContainerUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(Prim.GetPrimPath().GetString());
			FString ComponentDisplayLabel(Prim.GetName().ToString());

			UE::FUsdVariantSets VariantSets = Prim.GetVariantSets();
			const UE::FUsdVariantSet LODVariantSet = VariantSets.GetVariantSet(LODString);
			if (!LODVariantSet.IsValid())
			{
				return {};
			}

			const FString ActiveVariant = LODVariantSet.GetVariantSelection();
			if (!IsValidLODName(ActiveVariant))
			{
				return {};
			}

			if (UE::FUsdPrim ActiveLODMeshPrim = GetLODMesh(Prim, ActiveVariant))
			{
				ActiveVariantUSDPrimPath = ActiveLODMeshPrim.GetPrimPath().GetString();
			}

			if (const UInterchangeBaseNode* Node = NodeContainer->GetNode(LODContainerUid))
			{
				return LODContainerUid;
			}

			UE::FUsdStage UsdStage = UsdContext.GetUsdStage();

			UInterchangeMeshLODContainerNode* LODContainerComponentNode = NewObject<UInterchangeMeshLODContainerNode>(NodeContainer);
			NodeContainer->SetupNode(LODContainerComponentNode, LODContainerUid, ComponentDisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);

			AccumulatedInfo.PrimSceneNodes.Add(LODContainerComponentNode);

			FTraversalInfo InfoCopy = Info;
			InfoCopy.ParentNode = ParentSceneNode;
			InfoCopy.bInsideLODVariant = true;

			bool bSwitchedFromInitialVariant = false;
			TArray<FString> VariantNames = LODVariantSet.GetVariantNames();
			TSet<int32> ProcessedLODIndices;
			for (const FString& VariantName : VariantNames)
			{
				if (!IsValidLODName(VariantName))
				{
					continue;
				}

				{
					// For creating the scene nodes themselves we'll switch the active variant on the currently opened stage
					// (still using the session layer to minimize impact to the actual layer). This mainly so that we can retrieve
					// and fetch and cache the correct material bindings for the LOD meshes. Later on we'll use separate stages
					// with population masks to read the LODs concurrently, and we won't be able to resolve material bindings
					UE::FUsdEditContext Context{ UsdStage, UsdStage.GetSessionLayer() };

					bool bSwitchedVariant = VariantSets.SetSelection(LODString, VariantName);
					if (!bSwitchedVariant)
					{
						continue;
					}
					bSwitchedFromInitialVariant = true;
				}

				if (UE::FUsdPrim LODPrim = GetLODMesh(Prim, VariantName))
				{
					int32 LODIndex = GetLODIndexFromName(VariantName);
					if (ProcessedLODIndices.Contains(LODIndex))
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT("LodSubtreeDuplicateLOD", "Ignoring duplicate LOD Index {0} found in Prim '{1}'."),
							LODIndex,
							FText::FromString(Prim.GetPrimPath().GetString())
						));
						continue;
					}

					FHandlerAccumulatedInfo MeshPrimAccumulatedInfo;
					const bool bAllowSceneNodeGeneration = false;
					CurrentTranslator->TranslatePrimSubtree(LODPrim, InfoCopy, bAllowSceneNodeGeneration, &MeshPrimAccumulatedInfo);

					ProcessedLODIndices.Add(LODIndex);

					for (UInterchangeBaseNode* BaseNode : MeshPrimAccumulatedInfo.PrimAssetNodes)
					{
						if (UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode); MeshNode && !MeshNode->IsMorphTarget())
						{
							FString OriginalMeshPrimPath = UE::Interchange::USD::GetMeshPrimPath(*MeshNode);
								
							FTransform MeshTransform = UE::Interchange::USD::GetCombinedTransform(LODPrim, LODPrim.GetStage().GetPrimAtPath(UE::FSdfPath{ *OriginalMeshPrimPath }));

							LODContainerComponentNode->AddMeshForLODIndex(LODIndex, MeshNode->GetUniqueID(), MeshTransform);
						}
					}
				}
				else
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"InvalidLodMesh",
							"Failed to parse a LOD Mesh from variant '{0}' of prim '{1}'. For automatic parsing of LODs, make sure there is a single Mesh prim within the variant, named exactly as the variant itself (e.g. 'LOD0', 'LOD1', etc.)"
						),
						FText::FromString(VariantName),
						FText::FromString(Prim.GetPrimPath().GetString())
					));
				}
			}

			// Put the active variant back to what it originally was
			if (bSwitchedFromInitialVariant)
			{
				{
					UE::FUsdEditContext Context{UsdStage, UsdStage.GetSessionLayer()};

					const bool bRestoredSelection = VariantSets.SetSelection(LODString, ActiveVariant);
					ensure(bRestoredSelection);
				}

				// Recompute our skel cache here if we have any as ancestor, because switching variants could have
				// invalidated some of its internal state about its descendant prims, which we'll need to be OK
				// when handling the payloads
				Info.RepopulateSkelCache(UsdStage);
			}

			return LODContainerUid;
		}

		void TraverseNestedPointInstancerHierarchy(
			const UE::FUsdPrim& Prim,
			FTraversalInfo Info,
			const FTransform& ParentGlobalTransform
		)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TraverseNestedPointInstancerHierarchy)

			const FUsdStageInfo StageInfo{Prim.GetStage()};

			FTransform xFormTransform = FTransform::Identity;
			bool bResetTransformStack = false;
			UsdToUnreal::ConvertXformable(Prim.GetStage(), UE::FUsdTyped(Prim), xFormTransform, UsdUtils::GetEarliestTimeCode(), &bResetTransformStack);

			Info.UpdateWithCurrentPrim(Prim);

			FTransform CurrentGlobalTransform = xFormTransform * ParentGlobalTransform;

			if (Prim.IsA(TEXT("PointInstancer")))
			{
				TraversePointInstancerHierarchy(Prim, Info, CurrentGlobalTransform);
			}
			// TODO: LodSubtreeAPI support?
			else if (Info.bIsLODVariantContainer)
			{
				FString ActiveVariantUSDPrimPath;
				FString LODContainerUid = ConvertLodContainerPrim(Prim, ActiveVariantUSDPrimPath, Info);

				if (Info.bVisible)
				{
					UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = CreateSceneComponent(LODContainerUid, Prim);
					if (!ISMComponentNode)
					{
						return;
					}

					ISMComponentNode->AddInstanceTransform(CurrentGlobalTransform);
					if (LODContainerUid.Len())
					{
						ISMComponentNode->SetCustomInstancedAssetUid(LODContainerUid);
					}
				}

				TArray<FUsdPrim> Children = Prim.GetChildren();
				if (Children.Num())
				{
					for (FUsdPrim& Child : Children)
					{
						if (ActiveVariantUSDPrimPath == Child.GetPrimPath().GetString())
						{
							TArray<FUsdPrim> ActiveVariantChildren = Child.GetChildren();
							for (const FUsdPrim& ActiveVariantChild : ActiveVariantChildren)
							{
								TraverseNestedPointInstancerHierarchy(ActiveVariantChild, Info, CurrentGlobalTransform);
							}
						}
						else
						{
							TraverseNestedPointInstancerHierarchy(Child, Info, CurrentGlobalTransform);
						}
					}
				}
			}
			else if (Prim.IsA(TEXT("Mesh")))
			{
				if (Info.bVisible)
				{
					if (const UInterchangeMeshNode* MeshNode = TranslateMeshNode(Prim, Info))
					{
						const FString& MeshUid = MeshNode->GetUniqueID();

						//This could be a basic StaticMesh, but currently we only have support for InstancedStaticMeshes. (within the context of Components)
						UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<UInterchangeInstancedStaticMeshComponentNode>(CreateSceneComponent(MeshUid, Prim));
						if (!ISMComponentNode)
						{
							return;
						}

						ISMComponentNode->AddInstanceTransform(CurrentGlobalTransform);
						ISMComponentNode->SetCustomInstancedAssetUid(MeshUid);
					}
				}
			}
			else
			{
				TArray<FUsdPrim> Children = Prim.GetChildren();
				for (const FUsdPrim& Child : Children)
				{
					TraverseNestedPointInstancerHierarchy(Child, Info, CurrentGlobalTransform);
				}
			}
		}

		void TraversePointInstancerHierarchy(
			const UE::FUsdPrim& Prim,
			FTraversalInfo Info,
			const FTransform& ParentGlobalTransform
		)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TraversePointInstancerHierarchy)

			UsdUtils::FUsdPointInstancerInstancesData InstancesData;
			UsdUtils::GetPointInstancerInstancesData(Prim, InstancesData);

			const FUsdStageInfo StageInfo(Prim.GetStage());

			for (int32 PrototypeIndex = 0; PrototypeIndex < InstancesData.Prototypes.Num(); ++PrototypeIndex)
			{
				UsdUtils::FUsdPointInstancerInstancesData::FUsdPrototypeInstances& PrototypeInstanceData = InstancesData.Prototypes[PrototypeIndex];
				TArray<FTransform>& PrototypeInstanceTransforms = PrototypeInstanceData.InstanceTransforms;
				FString PrototypePathString = PrototypeInstanceData.PathString;

				UE::FUsdPrim PrototypePrim = Prim.GetStage().GetPrimAtPath(UE::FSdfPath{ *PrototypePathString });
				if (!PrototypePrim)
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT("MissingPrototype", "Failed to find prototype '{0}' for point instancer prim '{1}'"),
						FText::FromString(PrototypePathString),
						FText::FromString(Prim.GetPrimPath().GetString())
					));
					continue;
				}

				FTraversalInfo PrototypePrimInfo = Info;
				PrototypePrimInfo.UpdateWithCurrentPrim(PrototypePrim);

				if (PrototypePrim.IsA(TEXT("Mesh")))
				{
					if (PrototypePrimInfo.bVisible)
					{
						if (const UInterchangeMeshNode* MeshNode = TranslateMeshNode(PrototypePrim, PrototypePrimInfo))
						{
							FTransform xFormTransform = FTransform::Identity;
							bool bResetTransformStack = false;
							UsdToUnreal::ConvertXformable(PrototypePrim.GetStage(), UE::FUsdTyped(PrototypePrim), xFormTransform, UsdUtils::GetEarliestTimeCode(), &bResetTransformStack);

							for (FTransform& Transform : PrototypeInstanceTransforms)
							{
								Transform = xFormTransform * Transform * ParentGlobalTransform;
							}

							const FString& MeshUid = MeshNode->GetUniqueID();

							UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = CreateSceneComponent(MeshUid, PrototypePrim);
							if (!ISMComponentNode)
							{
								continue;
							}

							ISMComponentNode->AddInstanceTransforms(PrototypeInstanceTransforms);
							ISMComponentNode->SetCustomInstancedAssetUid(MeshUid);
						}
					}
				}
				else
				{
					for (int32 InstanceCounter = 0; InstanceCounter < PrototypeInstanceTransforms.Num(); ++InstanceCounter)
					{
						FTransform CurrentGlobalTransform = PrototypeInstanceTransforms[InstanceCounter] * ParentGlobalTransform;

						TraverseNestedPointInstancerHierarchy(PrototypePrim, PrototypePrimInfo, CurrentGlobalTransform);
					}
				}
			}
		}
	};
}	 // namespace UE::GeomPointInstancerSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FPointInstancerSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("PointInstancerHandler");
		return HandlerName;
	}

	const FString& FPointInstancerSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("PointInstancer");
		return SchemaName;
	}

	TOptional<bool> FPointInstancerSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// If we can't collapse point instancers then we already know for sure this can't be collapsed
		UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings();
		if (Settings && Settings->PointInstancerCollapsing == EUsdPointInstancerCollapsing::NoCollapsing)
		{
			return false;
		}

		return {};
	}

	TOptional<bool> FPointInstancerSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		// If we can't collapse point instancers then we already know for sure this can't collapse
		UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings();
		if (Settings && Settings->PointInstancerCollapsing == EUsdPointInstancerCollapsing::NoCollapsing)
		{
			return false;
		}

		return {};
	}

	bool FPointInstancerSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPointInstancerSchemaHandler::OnTranslate)

		using namespace UE::GeomPointInstancerSchemaHandler::Private;

		//We early out for LOD variants (for all Prims part of a LOD variant/LOD Group), 
		// otherwise we would query the info cache for inactive LOD Variant prims, which would trigger an ensure in DoesPathCollapseChildren.
		if (TraversalInfo.bIsLODVariantContainer || TraversalInfo.bInsideLODVariant)
		{
			return false;
		}

		// Don't spawn anything if this point instancer is collapsed
		if (FInterchangeUsdInfoCache* InfoCache = UsdContext.GetInterchangeInfoCache())
		{
			if (InfoCache->IsPathCollapsed(Prim.GetPrimPath()) || InfoCache->DoesPathCollapseChildren(Prim.GetPrimPath()))
			{
				return false;
			}
		}

		UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext));
		if (!SceneNode)
		{
			return false;
		}

		// This schema handler takes charge of recursive traversal, so let's prevent the UsdTranslator from visiting any children again.
		// We'll already get this done automatically for some prims as we call TranslatePrim() directly for e.g. Meshes, but we really
		// don't want the USD translator visiting any other of our children either
		constexpr bool bTraverseInstanceProxies = true;
		for (const FUsdPrim& ChildPrim : Prim.GetFilteredChildren(bTraverseInstanceProxies))
		{
			UsdContext.HandledPrimInfo.FindOrAdd(ChildPrim.GetPrimPath());
		}

		FISMComponentTraversalHelper ISMComponentTraversalHelper{UsdContext, AccumulatedInfo, SceneNode};
		ISMComponentTraversalHelper.TraversePointInstancerHierarchy(Prim, TraversalInfo, FTransform());
		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
