// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/SchemaHandler.h"

#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/HandlerAccumulatedInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"

#include "InterchangeGenericPayloadData.h"

#define LOCTEXT_NAMESPACE "UsdSchemaHandler"

namespace UE::Interchange::USD
{
	FSchemaHandler::~FSchemaHandler()
	{
	}

	bool FSchemaHandler::IsEnabled() const
	{
		return bEnabled;
	}

	void FSchemaHandler::SetEnabled(bool bInEnabled)
	{
		bEnabled = bInEnabled;
	}

	const FString& FSchemaHandler::GetTargetSchemaName() const
	{
		const static FString Empty;
		return Empty;
	}

	bool FSchemaHandler::CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::CanHandlePrim)

		const FString& SchemaName = GetTargetSchemaName();
		return Prim.IsA(*SchemaName);
	}

	TOptional<bool> FSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return {};
	}

	TOptional<bool> FSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return {};
	}

	bool FSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnTranslate)

		return false;
	}

	bool FSchemaHandler::OnGetMeshPayloadData(
		const FInterchangeMeshPayLoadKey& PayLoadKey,
		const UE::Interchange::FAttributeStorage& PayloadAttributes,
		UInterchangeUsdContext& UsdContext,

		TOptional<UE::Interchange::FMeshPayloadData>& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetMeshPayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportImage>& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetTexturePayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetBlockedTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportBlockedImage>& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetBlockedTexturePayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetAnimationPayloadData(
		const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetAnimationPayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetVolumePayloadData(
		const UE::Interchange::FVolumePayloadKey& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FVolumePayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetVolumePayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetGroomPayloadData(
		const FInterchangeGroomPayloadKey& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FGroomPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetGroomPayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetAudioPayloadData(
		const FString& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FInterchangeAudioPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetAudioPayloadData)

		return false;
	}

	bool FSchemaHandler::OnGetGenericPayloadData(
		const FString& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TObjectPtr<UInterchangeGenericPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetGenericPayloadData)

		return false;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
