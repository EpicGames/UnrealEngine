// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/SchemaHandlerRegistry.h"
#include "USDErrorUtils.h"

#define LOCTEXT_NAMESPACE "InterchangeUSDTranslator"

namespace UE::Interchange::USD
{
	TArray<FSchemaHandlerEntry> FSchemaHandlerRegistry::RegisteredHandlerEntries;
	TMap<FString, FSchemaHandlerGenerator> FSchemaHandlerRegistry::RegisteredHandlerNamesToGenerators;

	bool FSchemaHandlerRegistry::Unregister(const FString& Name)
	{
		uint64 NumRemoved = RegisteredHandlerEntries.RemoveAll(
			[Name](const FSchemaHandlerEntry& Rec)
			{
				return Rec.HandlerName == Name;
			}
		);

		uint64 NumGeneratorRemoved = RegisteredHandlerNamesToGenerators.Remove(Name);

		// We will validate duplicate entries on FSchemaHandlerRegistry::GenerateHandlers, in here a success
		// means having removed *anything*. This return value is not particularly checked by anything at this
		// point anyway...
		return NumRemoved > 0 || NumGeneratorRemoved > 0;
	}

	TArray<TSharedRef<FSchemaHandler>> FSchemaHandlerRegistry::GenerateHandlers(const TArray<FSchemaHandlerEntry>* Entries)
	{
		if (Entries == nullptr)
		{
			return {};
		}

		const uint64 NumEntries = Entries->Num();

		TArray<TSharedRef<FSchemaHandler>> Result;
		Result.Reserve(NumEntries);

		TSet<FString> SeenHandlerNames;
		SeenHandlerNames.Reserve(NumEntries);

		for (const FSchemaHandlerEntry& Entry : *Entries)
		{
			FSchemaHandlerGenerator* FoundGenerator = RegisteredHandlerNamesToGenerators.Find(Entry.HandlerName);
			if (!FoundGenerator)
			{
				USD_LOG_USERERROR(FText::Format(
					LOCTEXT("MissingHandlerGenerator", "Registered schema handler '{0}' does not have a generator function assigned."),
					FText::FromString(Entry.HandlerName)
				));
				continue;
			}

			// The generator will also apply our handler entry settings to the new handler instance
			TSharedRef<FSchemaHandler> NewHandler = (*FoundGenerator)(Entry);

			// We check for duplicate handler names here instead of in Registry because it's possible that user C++ code may have manipulated 
			// the registered handler array directly, adding/removing handlers (as it was intended) and skipping our own Register function.
			// This GenerateHandlers however is called by the USD Translator directly and is not affected by user code, so we know
			// we can validate this fully at this point.
			//
			// Also, the fact that this is normally called during translation means we should have the Interchange result container in place, and
			// the USD error utils should capture this error message and show the toast mentioning how there were issues with the import
			if (SeenHandlerNames.Contains(Entry.HandlerName))
			{
				USD_LOG_USERERROR(FText::Format(
					LOCTEXT(
						"DuplicateHandlerName",
						"Registered schema handlers should have unique names, but name '{0}' is used more than once. Translation may be inaccurate."
					),
					FText::FromString(Entry.HandlerName)
				));
			}
			SeenHandlerNames.Add(Entry.HandlerName);

			Result.Add(NewHandler);
		}

		return Result;
	};
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
