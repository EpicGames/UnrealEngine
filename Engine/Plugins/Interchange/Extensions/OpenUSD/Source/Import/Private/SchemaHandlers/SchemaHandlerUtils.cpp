// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/SchemaHandlerUtils.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/SchemaHandlerEntry.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDAttributeUtils.h"
#include "USDErrorUtils.h"
#include "USDNaniteAssemblyUtils.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdEditContext.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelBinding.h"
#include "UsdWrappers/UsdSkelBlendShape.h"
#include "UsdWrappers/UsdSkelBlendShapeQuery.h"
#include "UsdWrappers/UsdSkelCache.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"
#include "UsdWrappers/UsdVariantSets.h"
#include "UsdWrappers/VtValue.h"

#include "MaterialX/InterchangeMaterialXDefinitions.h"
#include "USDMaterialXShaderGraph.h"

#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMaterialReferenceNode.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "Internationalization/Regex.h"
#include "MovieSceneSection.h"
#include "StaticMeshAttributes.h"

#define LOCTEXT_NAMESPACE "SchemaHandlerUtils"

namespace UE::USDSchemaHandlerUtils::Private
{
	using namespace UE::Interchange::USD;

	FString AddUnrealMaterialReferenceNodeIfNeeded(
		const FString& ContentPath,
		UInterchangeBaseNodeContainer& NodeContainer,
		FHandlerAccumulatedInfo& AccumulatedInfo
	)
	{
		using namespace UsdUnreal::MaterialUtils;

		// e.g. "\\MaterialReference\\/Game/MyFolder/Red.Red"
		const FString NodeUid = MakeNodeUid(MaterialReferencePrefix + ContentPath);

		UInterchangeMaterialReferenceNode* Node = GetExistingNode<UInterchangeMaterialReferenceNode>(NodeContainer, NodeUid);
		if (!Node)
		{
			const FString DisplayName = FPaths::GetBaseFilename(ContentPath);

			Node = NewObject<UInterchangeMaterialReferenceNode>(&NodeContainer);
			NodeContainer.SetupNode(Node, NodeUid, DisplayName, EInterchangeNodeContainerType::TranslatedAsset);
		}

		Node->SetCustomContentPath(ContentPath);

		AccumulatedInfo.PrimAssetNodes.Add(Node);

		return NodeUid;
	}

	template<typename USDType, typename InterchangeType = USDType>
	InterchangeType CoerceToInterchangeType(const USDType& USDValue)
	{
		if constexpr (std::is_same_v<USDType, FMatrix2D> && std::is_same_v<InterchangeType, FMatrix44d>)
		{
			FMatrix44d Result{EForceInit::ForceInitToZero};
			Result.M[0][0] = USDValue.Row0.X;
			Result.M[0][1] = USDValue.Row0.Y;

			Result.M[1][0] = USDValue.Row1.X;
			Result.M[1][1] = USDValue.Row1.Y;
			return Result;
		}
		else if constexpr (std::is_same_v<USDType, FMatrix3D> && std::is_same_v<InterchangeType, FMatrix44d>)
		{
			FMatrix44d Result{EForceInit::ForceInitToZero};
			Result.M[0][0] = USDValue.Row0.X;
			Result.M[0][1] = USDValue.Row0.Y;
			Result.M[0][2] = USDValue.Row0.Z;

			Result.M[1][0] = USDValue.Row1.X;
			Result.M[1][1] = USDValue.Row1.Y;
			Result.M[1][2] = USDValue.Row1.Z;

			Result.M[2][0] = USDValue.Row2.X;
			Result.M[2][1] = USDValue.Row2.Y;
			Result.M[2][2] = USDValue.Row2.Z;
			return Result;
		}
		else
		{
			return (InterchangeType)USDValue;
		}
	};

	template<typename USDType, typename InterchangeType = USDType>
	void AddUserAttribute(const FString& KeyName, const UE::FVtValue& VtValue, UInterchangeBaseNode* Node)
	{
		const USDType USDValue = VtValue.Get<USDType>();
		const TOptional<FString> PayloadKey;

		if constexpr (std::is_same_v<InterchangeType, USDType>)
		{
			UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(Node, KeyName, USDValue, PayloadKey);
		}
		else if constexpr (TIsTArray<USDType>::Value)
		{
			static_assert(TIsTArray<InterchangeType>::Value);

			InterchangeType InterchangeArray;
			InterchangeArray.SetNum(USDValue.Num());
			for (int32 Index = 0; Index < USDValue.Num(); ++Index)
			{
				InterchangeArray[Index] = CoerceToInterchangeType<typename USDType::ElementType, typename InterchangeType::ElementType>(
					USDValue[Index]
				);
			}

			UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(Node, KeyName, InterchangeArray, PayloadKey);
		}
		else
		{
			const InterchangeType InterchangeValue = CoerceToInterchangeType<USDType, InterchangeType>(USDValue);
			UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(Node, KeyName, InterchangeValue, PayloadKey);
		}
	};

	// Note: We could have just used the Cpp type names here instead of the value type token, but doing the
	// latter lets us handle the color types as FLinearColor, which is probably more useful
	// References:
	// - https://openusd.org/docs/api/_usd__page__datatypes.html
	// - EAttributeTypes declaration
	// - Engine/Source/Runtime/Interchange/Core/Private/Tests/StorageTest.cpp
	using AddUserAttributeFunc = TFunction<void(const FString&, const UE::FVtValue&, UInterchangeBaseNode*)>;
	static TMap<FName, AddUserAttributeFunc> AddUserAttributeFuncs = {
		{TEXT("bool"), AddUserAttribute<bool>},

		{TEXT("uchar"), AddUserAttribute<uint8>},
		{TEXT("unsigned char"), AddUserAttribute<uint8>},
		{TEXT("int"), AddUserAttribute<int32>},
		{TEXT("unsigned int"), AddUserAttribute<uint32>},
		{TEXT("uint"), AddUserAttribute<uint32>},
		{TEXT("int64"), AddUserAttribute<int64>},
		{TEXT("__int64"), AddUserAttribute<int64>},
		{TEXT("unsigned __int64"), AddUserAttribute<uint64>},
		{TEXT("uint64"), AddUserAttribute<uint64>},

		{TEXT("half"), AddUserAttribute<FFloat16>},
		{TEXT("pxr_half::half"), AddUserAttribute<FFloat16>},
		{TEXT("float"), AddUserAttribute<float>},
		{TEXT("double"), AddUserAttribute<double>},
		{TEXT("timecode"), AddUserAttribute<FSdfTimeCode, double>},
		{TEXT("SdfTimeCode"), AddUserAttribute<FSdfTimeCode, double>},

		{TEXT("string"), AddUserAttribute<FString>},
		{TEXT("token"), AddUserAttribute<FName>},
		{TEXT("TfToken"), AddUserAttribute<FName>},
		{TEXT("asset"), AddUserAttribute<FSdfAssetPath, FString>},
		{TEXT("SdfAssetPath"), AddUserAttribute<FSdfAssetPath, FString>},

		{TEXT("matrix2d"), AddUserAttribute<FMatrix2D, FMatrix44d>},
		{TEXT("GfMatrix2d"), AddUserAttribute<FMatrix2D, FMatrix44d>},
		{TEXT("matrix3d"), AddUserAttribute<FMatrix3D, FMatrix44d>},
		{TEXT("GfMatrix3d"), AddUserAttribute<FMatrix3D, FMatrix44d>},
		{TEXT("matrix4d"), AddUserAttribute<FMatrix44d>},
		{TEXT("GfMatrix4d"), AddUserAttribute<FMatrix44d>},
		{TEXT("frame4d"), AddUserAttribute<FMatrix44d>},

		{TEXT("quath"), AddUserAttribute<FQuat4h, FQuat4f>},
		{TEXT("GfQuath"), AddUserAttribute<FQuat4h, FQuat4f>},
		{TEXT("quatf"), AddUserAttribute<FQuat4f>},
		{TEXT("GfQuatf"), AddUserAttribute<FQuat4f>},
		{TEXT("quatd"), AddUserAttribute<FQuat4d>},
		{TEXT("GfQuatd"), AddUserAttribute<FQuat4d>},

		{TEXT("half2"), AddUserAttribute<FVector2DHalf>},
		{TEXT("GfVec2h"), AddUserAttribute<FVector2DHalf>},
		{TEXT("float2"), AddUserAttribute<FVector2f>},
		{TEXT("GfVec2f"), AddUserAttribute<FVector2f>},
		{TEXT("double2"), AddUserAttribute<FVector2d>},
		{TEXT("GfVec2d"), AddUserAttribute<FVector2d>},
		{TEXT("int2"), AddUserAttribute<FIntPoint>},
		{TEXT("GfVec2i"), AddUserAttribute<FIntPoint>},

		{TEXT("half3"), AddUserAttribute<FVector3h, FVector3f>},
		{TEXT("GfVec3h"), AddUserAttribute<FVector3h, FVector3f>},
		{TEXT("point3h"), AddUserAttribute<FVector3h, FVector3f>},
		{TEXT("normal3h"), AddUserAttribute<FVector3h, FVector3f>},
		{TEXT("vector3h"), AddUserAttribute<FVector3h, FVector3f>},
		{TEXT("color3h"), AddUserAttribute<FVector3h, FLinearColor>},

		{TEXT("float3"), AddUserAttribute<FVector3f>},
		{TEXT("GfVec3f"), AddUserAttribute<FVector3f>},
		{TEXT("point3f"), AddUserAttribute<FVector3f>},
		{TEXT("normal3f"), AddUserAttribute<FVector3f>},
		{TEXT("vector3f"), AddUserAttribute<FVector3f>},
		{TEXT("color3f"), AddUserAttribute<FVector3f, FLinearColor>},

		{TEXT("double3"), AddUserAttribute<FVector3d>},
		{TEXT("GfVec3d"), AddUserAttribute<FVector3d>},
		{TEXT("point3d"), AddUserAttribute<FVector3d>},
		{TEXT("normal3d"), AddUserAttribute<FVector3d>},
		{TEXT("vector3d"), AddUserAttribute<FVector3d>},
		{TEXT("color3d"), AddUserAttribute<FVector3d, FLinearColor>},

		{TEXT("int3"), AddUserAttribute<FIntVector>},
		{TEXT("GfVec3i"), AddUserAttribute<FIntVector>},

		{TEXT("half4"), AddUserAttribute<FVector4h, FVector4f>},
		{TEXT("GfVec4h"), AddUserAttribute<FVector4h, FVector4f>},
		{TEXT("float4"), AddUserAttribute<FVector4f>},
		{TEXT("GfVec4f"), AddUserAttribute<FVector4f>},
		{TEXT("double4"), AddUserAttribute<FVector4d>},
		{TEXT("GfVec4d"), AddUserAttribute<FVector4d>},
		{TEXT("int4"), AddUserAttribute<FIntRect>},
		{TEXT("GfVec4i"), AddUserAttribute<FIntRect>},
		{TEXT("color4h"), AddUserAttribute<FVector4h, FLinearColor>},
		{TEXT("color4f"), AddUserAttribute<FVector4f, FLinearColor>},
		{TEXT("color4d"), AddUserAttribute<FVector4d, FLinearColor>},

		{TEXT("bool[]"), AddUserAttribute<TArray<bool>>},
		{TEXT("VtArray<bool>"), AddUserAttribute<TArray<bool>>},

		{TEXT("uchar[]"), AddUserAttribute<TArray<uint8>>},
		{TEXT("VtArray<unsigned char>"), AddUserAttribute<TArray<uint8>>},
		{TEXT("int[]"), AddUserAttribute<TArray<int32>>},
		{TEXT("VtArray<int>"), AddUserAttribute<TArray<int32>>},
		{TEXT("uint[]"), AddUserAttribute<TArray<uint32>>},
		{TEXT("VtArray<unsigned int>"), AddUserAttribute<TArray<uint32>>},
		{TEXT("int64[]"), AddUserAttribute<TArray<int64>>},
		{TEXT("VtArray<__int64>"), AddUserAttribute<TArray<int64>>},
		{TEXT("uint64[]"), AddUserAttribute<TArray<uint64>>},
		{TEXT("VtArray<unsigned __int64>"), AddUserAttribute<TArray<uint64>>},

		{TEXT("half[]"), AddUserAttribute<TArray<FFloat16>>},
		{TEXT("VtArray<pxr_half::half>"), AddUserAttribute<TArray<FFloat16>>},
		{TEXT("float[]"), AddUserAttribute<TArray<float>>},
		{TEXT("VtArray<float>"), AddUserAttribute<TArray<float>>},
		{TEXT("double[]"), AddUserAttribute<TArray<double>>},
		{TEXT("VtArray<double>"), AddUserAttribute<TArray<double>>},
		{TEXT("timecode[]"), AddUserAttribute<TArray<FSdfTimeCode>, TArray<double>>},
		{TEXT("VtArray<SdfTimeCode>"), AddUserAttribute<TArray<FSdfTimeCode>, TArray<double>>},

		{TEXT("string[]"), AddUserAttribute<TArray<FString>>},
		{TEXT("VtArray<string>"), AddUserAttribute<TArray<FString>>},
		{TEXT("token[]"), AddUserAttribute<TArray<FName>>},
		{TEXT("VtArray<TfToken>"), AddUserAttribute<TArray<FName>>},
		{TEXT("asset[]"), AddUserAttribute<TArray<FSdfAssetPath>, TArray<FString>>},
		{TEXT("VtArray<SdfAssetPath>"), AddUserAttribute<TArray<FSdfAssetPath>, TArray<FString>>},

		{TEXT("matrix2d[]"), AddUserAttribute<TArray<FMatrix2D>, TArray<FMatrix44d>>},
		{TEXT("VtArray<GfMatrix2d>"), AddUserAttribute<TArray<FMatrix2D>, TArray<FMatrix44d>>},
		{TEXT("matrix3d[]"), AddUserAttribute<TArray<FMatrix3D>, TArray<FMatrix44d>>},
		{TEXT("VtArray<GfMatrix3d>"), AddUserAttribute<TArray<FMatrix3D>, TArray<FMatrix44d>>},
		{TEXT("matrix4d[]"), AddUserAttribute<TArray<FMatrix44d>>},
		{TEXT("VtArray<GfMatrix4d>"), AddUserAttribute<TArray<FMatrix44d>>},
		{TEXT("frame4d[]"), AddUserAttribute<TArray<FMatrix44d>>},

		{TEXT("quath[]"), AddUserAttribute<TArray<FQuat4h>, TArray<FQuat4f>>},
		{TEXT("VtArray<GfQuath>"), AddUserAttribute<TArray<FQuat4h>, TArray<FQuat4f>>},
		{TEXT("quatf[]"), AddUserAttribute<TArray<FQuat4f>>},
		{TEXT("VtArray<GfQuatf>"), AddUserAttribute<TArray<FQuat4f>>},
		{TEXT("quatd[]"), AddUserAttribute<TArray<FQuat4d>>},
		{TEXT("VtArray<GfQuatd>"), AddUserAttribute<TArray<FQuat4d>>},

		{TEXT("half2[]"), AddUserAttribute<TArray<FVector2DHalf>>},
		{TEXT("VtArray<GfVec2h>"), AddUserAttribute<TArray<FVector2DHalf>>},
		{TEXT("float2[]"), AddUserAttribute<TArray<FVector2f>>},
		{TEXT("VtArray<GfVec2f>"), AddUserAttribute<TArray<FVector2f>>},
		{TEXT("double2[]"), AddUserAttribute<TArray<FVector2d>>},
		{TEXT("VtArray<GfVec2d>"), AddUserAttribute<TArray<FVector2d>>},
		{TEXT("int2[]"), AddUserAttribute<TArray<FIntPoint>>},
		{TEXT("VtArray<GfVec2i>"), AddUserAttribute<TArray<FIntPoint>>},

		{TEXT("half3[]"), AddUserAttribute<TArray<FVector3h>, TArray<FVector3f>>},
		{TEXT("VtArray<GfVec3h>"), AddUserAttribute<TArray<FVector3h>, TArray<FVector3f>>},
		{TEXT("point3h[]"), AddUserAttribute<TArray<FVector3h>, TArray<FVector3f>>},
		{TEXT("normal3h[]"), AddUserAttribute<TArray<FVector3h>, TArray<FVector3f>>},
		{TEXT("vector3h[]"), AddUserAttribute<TArray<FVector3h>, TArray<FVector3f>>},
		{TEXT("color3h[]"), AddUserAttribute<TArray<FVector3h>, TArray<FLinearColor>>},

		{TEXT("float3[]"), AddUserAttribute<TArray<FVector3f>>},
		{TEXT("VtArray<GfVec3f>"), AddUserAttribute<TArray<FVector3f>>},
		{TEXT("point3f[]"), AddUserAttribute<TArray<FVector3f>>},
		{TEXT("normal3f[]"), AddUserAttribute<TArray<FVector3f>>},
		{TEXT("vector3f[]"), AddUserAttribute<TArray<FVector3f>>},
		{TEXT("color3f[]"), AddUserAttribute<TArray<FVector3f>, TArray<FLinearColor>>},

		{TEXT("double3[]"), AddUserAttribute<TArray<FVector3d>>},
		{TEXT("VtArray<GfVec3d>"), AddUserAttribute<TArray<FVector3d>>},
		{TEXT("point3d[]"), AddUserAttribute<TArray<FVector3d>>},
		{TEXT("normal3d[]"), AddUserAttribute<TArray<FVector3d>>},
		{TEXT("vector3d[]"), AddUserAttribute<TArray<FVector3d>>},
		{TEXT("color3d[]"), AddUserAttribute<TArray<FVector3d>, TArray<FLinearColor>>},

		{TEXT("int3[]"), AddUserAttribute<TArray<FIntVector>>},
		{TEXT("VtArray<GfVec3i>"), AddUserAttribute<TArray<FIntVector>>},

		{TEXT("half4[]"), AddUserAttribute<TArray<FVector4h>, TArray<FVector4f>>},
		{TEXT("VtArray<GfVec4h>"), AddUserAttribute<TArray<FVector4h>, TArray<FVector4f>>},
		{TEXT("float4[]"), AddUserAttribute<TArray<FVector4f>>},
		{TEXT("VtArray<GfVec4f>"), AddUserAttribute<TArray<FVector4f>>},
		{TEXT("double4[]"), AddUserAttribute<TArray<FVector4d>>},
		{TEXT("VtArray<GfVec4d>"), AddUserAttribute<TArray<FVector4d>>},
		{TEXT("int4[]"), AddUserAttribute<TArray<FIntRect>>},
		{TEXT("VtArray<GfVec4i>"), AddUserAttribute<TArray<FIntRect>>},
		{TEXT("color4h[]"), AddUserAttribute<TArray<FVector4h>, TArray<FLinearColor>>},
		{TEXT("color4f[]"), AddUserAttribute<TArray<FVector4f>, TArray<FLinearColor>>},
		{TEXT("color4d[]"), AddUserAttribute<TArray<FVector4d>, TArray<FLinearColor>>},
	};
}	 // namespace UE::USDSchemaHandlerUtils::Private

namespace UE::Interchange::USD
{
	FString GenerateHash(const FString& NodeUid)
	{
		return LexToString(FCrc::StrCrc32(*NodeUid));
	}

	// Append a case sensistive hash of NodeUid to the NodeUid itself, making it so that any TMap<FString, T> that stores
	// these IDs behaves in a case sensitive manner.
	//
	// This is needed because unfortunately the default TMap<FString, T> is not case sensitive on the key FStrings, while prim
	// names are case sensitive. Even if we could modify the UInterchangeBaseNodeContainer::Nodes map to be case sensitive, we'd
	// still constantly get issues as any other TMap<FString, T> in the codebase that stores NodeUids would show collisions
	FString MakeNodeUid(const FString& NodeUid)
	{
		return NodeUid + TEXT("_") + GenerateHash(NodeUid);
	}

	//In the MakeNodeUid function we add a hash to the NodeUid.
	// In this function we remove this postfix to acquire the original MeshPrimPath.
	//	Also removes the \\Mesh\\ prefix.
	FString GetMeshPrimPathFromNodeUid(const FString& NodeUid)
	{
		FString PrimPath = NodeUid;
		
		int32 LastCharIndex = 0;
		if (PrimPath.FindLastChar('_', LastCharIndex))
		{
			FString HashPart = PrimPath.RightChop(LastCharIndex + 1);
			FString PrimPathCandidate = PrimPath.Left(LastCharIndex);
			FString PrimPathCandidateHash = GenerateHash(PrimPathCandidate);
			if (HashPart == PrimPathCandidateHash)
			{
				PrimPath = PrimPathCandidate;
			}
		}

		PrimPath.RemoveFromStart(MeshPrefix, ESearchCase::CaseSensitive);
		
		return PrimPath;
	}

	FString MakeBoneNodeUid(const FString& SkeletonPrimPath, const FString& ConcatBonePath)
	{
		return MakeNodeUid(BonePrefix + SkeletonPrimPath + TEXT("/") + ConcatBonePath);
	}

	FString MakeRootBoneNodeUid(const FString& SkeletonPrimPath)
	{
		return MakeBoneNodeUid(BonePrefix + SkeletonPrimPath, RootBoneUidSuffix);
	}

	FString GetMorphTargetMeshNodeUid(
		const UInterchangeUsdContext& UsdContext,
		const UE::FUsdPrim& MeshPrim,
		int32 MeshBlendShapeIndex,
		const FString& InbetweenName
	)
	{
		const FString Suffix = FString::Printf(TEXT("\\%d\\%s"), MeshBlendShapeIndex, *InbetweenName);
		return UsdContext.MakeAssetNodeUid(MeshPrim, MorphTargetPrefix, Suffix);
	}

	// Deprecated
	FString GetMorphTargetMeshNodeUid(const FString& MeshPrimPath, int32 MeshBlendShapeIndex, const FString& InbetweenName)
	{
		return MakeNodeUid(FString::Printf(TEXT("%s%s\\%d\\%s"), *MorphTargetPrefix, *MeshPrimPath, MeshBlendShapeIndex, *InbetweenName));
	}

	UInterchangeSceneNode* GetOrCreateDefaultSceneNode(const UE::FUsdPrim& Prim, const FTraversalInfo& TraversalInfo, UInterchangeUsdContext& UsdContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::GetOrCreateDefaultSceneNode)

		using namespace UE::InterchangeUsdTranslator::Private;

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return nullptr;
		}

		if (!Prim)
		{
			return nullptr;
		}

		const FString SceneNodeUid = UsdContext.MakeSceneNodeUid(Prim);
		const FName TypeName = Prim.GetTypeName();

		// Only prims that require rendering (and have a renderable parent) get a scene node.
		// This includes Xforms but also Scopes, which are not Xformable.
		// Also allow typeless prims to get a scene node otherwise some assets like geometry cache
		// would not get processed and they need to bake the transforms into the meshes.
		//
		// We add a scene node for the pseudoroot in order to make tree traversal easier on the pipeline,
		// but the pipeline will strip the pseudoroot node as its final step (if desired)
		const bool bIsImageable = Prim.IsA(TEXT("Imageable"));
		const bool bIsTypeless = TypeName.IsNone();
		const bool bNeedsSceneNode = Prim.IsPseudoRoot() || bIsTypeless
									 || (bIsImageable && (TraversalInfo.ParentNode || Prim.GetParent().IsPseudoRoot()))
									 || TraversalInfo.bIsLODVariantContainer;	 // We allow typeless prims (so not imageable) to be LOD containers

		UInterchangeSceneNode* SceneNode = GetExistingNode<UInterchangeSceneNode>(*NodeContainer, SceneNodeUid);
		if (!SceneNode)
		{
			FString DisplayLabel;
			if (Prim.IsPseudoRoot())
			{
				DisplayLabel = FPaths::GetBaseFilename(Prim.GetStage().GetRootLayer().GetDisplayName());
			}
			else
			{
				DisplayLabel = Prim.HasAuthoredDisplayName() ? Prim.GetDisplayName() : Prim.GetName().ToString();
			}

			SceneNode = NewObject<UInterchangeSceneNode>(NodeContainer);
			NodeContainer->SetupNode(
				SceneNode,
				SceneNodeUid,
				DisplayLabel,
				EInterchangeNodeContainerType::TranslatedScene,
				TraversalInfo.ParentNode ? TraversalInfo.ParentNode->GetUniqueID() : TEXT("")
			);

			// Store our prim kind as well, if we have any (this becomes the empty string if the prim has no authored kind)
			const FString KindString = UsdToUnreal::ConvertToken(IUsdPrim::GetKind(Prim));
			if (!KindString.IsEmpty())
			{
				const FString UserDefinedAttributeName = TEXT("kind");
				TOptional<FString> PayloadKey;
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, UserDefinedAttributeName, KindString, PayloadKey);
			}

			// Always ensure the scene node has a local transform or else UInterchangeSceneNode::GetGlobalTransformInternal() will fail to compute
			// the full global transform for child scene nodes, if no other handler ends up producing a local transform
			// TODO: Should this be changed on the Interchange side?
			SceneNode->SetCustomLocalTransform(NodeContainer, FTransform::Identity);

			if (UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings())
			{
				if (Settings->bTranslatePrimAttributes)
				{
					TranslateAttributes(Prim, SceneNode, Settings->AttributeRegexFilter);
				}

				if (Settings->bTranslatePrimMetadata)
				{
					TranslateMetadata(Prim, SceneNode, Settings->MetadataRegexFilter);
				}
			}
		}

		return SceneNode;
	}

	void TranslateAttributeAsUserAttribute(
		const UE::FUsdPrim& Prim,
		const FString& AttributeName,
		UInterchangeBaseNode* Node,
		const double TimeCode,
		bool bAuthoredOnly
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::TranslateAttributeAsUserAttribute)

		using namespace UE::USDSchemaHandlerUtils::Private;

		if (!Node || !Prim)
		{
			return;
		}


		UE::FUsdAttribute Attribute = Prim.GetAttribute(*AttributeName);
		if (!Attribute)
		{
			return;
		}

		if (bAuthoredOnly)
		{
			if (!Attribute.HasAuthoredValue())
			{
				return;
			}

			if (TimeCode == UsdUtils::GetDefaultTimeCode() && !UsdUtils::HasAuthoredDefaultOpinion(Attribute))
			{
				return;
			}
		}

		if (AddUserAttributeFunc Func = AddUserAttributeFuncs.FindRef(Attribute.GetTypeName()))
		{
			UE::FVtValue Value;
			if (Attribute.Get(Value, TimeCode))
			{
				Func(AttributeName, Value, Node);
			}
		}
	};

	void TranslateAttributes(const UE::FUsdPrim& Prim, UInterchangeBaseNode* Node, const FString& AllowedAttributeRegex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::TranslateAttributes)

		using namespace UE::USDSchemaHandlerUtils::Private;

		FRegexPattern RegexPattern{AllowedAttributeRegex};

		for (const UE::FUsdAttribute& Attr : Prim.GetAttributes())
		{
			if (!Attr.HasAuthoredValue())
			{
				continue;
			}

			const FString AttrName = Attr.GetName().ToString();

			FRegexMatcher RegexMatcher{RegexPattern, AttrName};
			if (!RegexMatcher.FindNext())
			{
				continue;
			}

			if (AddUserAttributeFunc Func = AddUserAttributeFuncs.FindRef(Attr.GetTypeName()))
			{
				UE::FVtValue Value;
				if (Attr.Get(Value))	// Always check for an opinion on the default time code
				{
					Func(AttrName, Value, Node);
				}
			}
		}
	}

	bool TranslateAPISchemaAttributes(const UE::FUsdPrim& Prim, const FString& AppliedAPISchemaName, UInterchangeBaseNode* Node)
	{
		if (Prim && Prim.HasAPI(*AppliedAPISchemaName))
		{
			for (const FString& PropertyName : UsdUtils::GetAppliedAPISchemaPropertyNames(AppliedAPISchemaName))
			{
				TranslateAttributeAsUserAttribute(Prim, PropertyName, Node);
			}

			return true;
		}

		return false;
	}

	void TranslateMetadata(const UE::FUsdPrim& Prim, UInterchangeBaseNode* Node, const FString& AllowedMetadataRegex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::TranslateMetadata)

		using namespace UE::USDSchemaHandlerUtils::Private;

		FRegexPattern RegexPattern{AllowedMetadataRegex};

		TFunction<void(const TMap<FString, UE::FVtValue>&, const FString&)> ParseMetadataMap;
		ParseMetadataMap = [&ParseMetadataMap, &RegexPattern, Node](const TMap<FString, UE::FVtValue>& MetadataMap, const FString& ConcatParentKey)
		{
			for (const TPair<FString, UE::FVtValue>& Pair : MetadataMap)
			{
				const FString FullKey = ConcatParentKey.IsEmpty() ? Pair.Key : FString::Printf(TEXT("%s:%s"), *ConcatParentKey, *Pair.Key);
				const FString TypeName = Pair.Value.GetTypeName();

				if (TypeName == TEXT("VtDictionary"))
				{
					TMap<FString, UE::FVtValue> NestedMap = Pair.Value.Get<TMap<FString, UE::FVtValue>>();
					ParseMetadataMap(NestedMap, FullKey);
				}
				else if (AddUserAttributeFunc Func = AddUserAttributeFuncs.FindRef(*TypeName))
				{
					// Only match when parsing leaf keys, because a key may not match just the parent nested map key directly and
					// would have otherwise prevented us from stepping into it (e.g. the regex "customData:my" will not match
					// "customData", but it would match "customData:myNestedMap")
					FRegexMatcher RegexMatcher{RegexPattern, FullKey};
					if (!RegexMatcher.FindNext())
					{
						continue;
					}

					Func(FullKey, Pair.Value, Node);
				}
			}
		};

		const FString ConcatParentKey;
		ParseMetadataMap(Prim.GetAllAuthoredMetadata(), ConcatParentKey);
	}

	void AddTransformAnimationNode(const UE::FUsdPrim& Prim, FHandlerAccumulatedInfo& AccumulatedInfo, UInterchangeUsdContext& UsdContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::AddTransformAnimationNode)

		const FString PrimPath = Prim.GetPrimPath().GetString();
		const FString UniquePath = PrimPath + TEXT("\\") + UnrealIdentifiers::TransformPropertyName.ToString();
		const FString AnimTrackNodeUid = MakeNodeUid(AnimationTrackPrefix + UniquePath);
		const FString SceneNodeUid = UsdContext.MakeSceneNodeUid(Prim);

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		UInterchangeTransformAnimationTrackNode* TrackNode = GetExistingNode<UInterchangeTransformAnimationTrackNode>(
			*NodeContainer,
			AnimTrackNodeUid
		);
		if (!TrackNode)
		{
			TrackNode = NewObject<UInterchangeTransformAnimationTrackNode>(NodeContainer);
			NodeContainer->SetupNode(TrackNode, AnimTrackNodeUid, UniquePath, EInterchangeNodeContainerType::TranslatedAsset);
		}

		TrackNode->SetCustomActorDependencyUid(SceneNodeUid);
		TrackNode->SetCustomAnimationPayloadKey(UniquePath, EInterchangeAnimationPayLoadType::CURVE);
		TrackNode->SetCustomUsedChannels((int32)EMovieSceneTransformChannel::AllTransform);

		AccumulatedInfo.PrimAssetNodes.Add(TrackNode);

		UsdContext.SetupTrackSetNode();
		UsdContext.CurrentTrackSet->AddCustomAnimationTrackUid(AnimTrackNodeUid);
	}

	void AddPropertyAnimationNode(
		const FString& PrimPath,
		const FName& UEPropertyName,
		EInterchangePropertyTracks TrackType,
		EInterchangeAnimationPayLoadType PayloadType,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		// We don't use the USD attribute path here because we want one unique node per UE track name,
		// so that if e.g. both "intensity" and "exposure" are animated we make a single track for
		// the Intensity UE property. Plus, when retrieving the payloads we will use the UEPropertyName as well
		const FString PayloadKey = PrimPath + TEXT("\\") + UEPropertyName.ToString();
		UE::FUsdPrim AnimPrim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath(*PrimPath));
		const FString SceneNodeUid = AnimPrim ? UsdContext.MakeSceneNodeUid(AnimPrim) : MakeNodeUid(PrimPath);
		const FString AnimTrackNodeUid = MakeNodeUid(AnimationTrackPrefix + PayloadKey);

		UInterchangeAnimationTrackNode* TrackNode = GetExistingNode<UInterchangeAnimationTrackNode>(*NodeContainer, AnimTrackNodeUid);
		if (!TrackNode)
		{
			TrackNode = NewObject<UInterchangeAnimationTrackNode>(NodeContainer);
			NodeContainer->SetupNode(TrackNode, AnimTrackNodeUid, PayloadKey, EInterchangeNodeContainerType::TranslatedAsset);
		}

		TrackNode->SetCustomActorDependencyUid(SceneNodeUid);
		TrackNode->SetCustomPropertyTrack(TrackType);
		TrackNode->SetCustomAnimationPayloadKey(PayloadKey, PayloadType);

		AccumulatedInfo.PrimAssetNodes.Add(TrackNode);

		UsdContext.SetupTrackSetNode();
		UsdContext.CurrentTrackSet->AddCustomAnimationTrackUid(AnimTrackNodeUid);
	}

	void AddNodesForAnimatedAttributes(
		const UE::FUsdPrim& Prim,
		const TMap<FString, TArray<FInterchangeTrackInfo>>& UsdAttributeNameToTrackInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::AddNodesForAnimatedAttributes)

		using namespace UE::InterchangeUsdTranslator::Private;

		if (!Prim)
		{
			return;
		}
		const FString PrimPath = Prim.GetPrimPath().GetString();

		for (const TPair<FString, TArray<FInterchangeTrackInfo>>& Pair : UsdAttributeNameToTrackInfo)
		{
			const FString& UsdAttributeName = Pair.Key;
			const TArray<FInterchangeTrackInfo>& TrackInfos = Pair.Value;

			UE::FUsdAttribute Attr = Prim.GetAttribute(*UsdAttributeName);
			if (!Attr || !Attr.ValueMightBeTimeVarying() || Attr.GetNumTimeSamples() == 0)
			{
				continue;
			}

			for (const FInterchangeTrackInfo& TrackInfo : TrackInfos)
			{
				// Note: Implicit assumption that PrimPath == SceneNodeUid
				AddPropertyAnimationNode(
					PrimPath,
					TrackInfo.PropertyName,
					TrackInfo.TrackType,
					TrackInfo.PayloadType,
					AccumulatedInfo,
					UsdContext
				);
			}
		}
	}

	bool ReadBools(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<bool(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		OutPayloadData.StepCurves.SetNum(1);
		FInterchangeStepCurve& Curve = OutPayloadData.StepCurves[0];
		TArray<float>& KeyTimes = Curve.KeyTimes;
		TArray<bool>& BooleanKeyValues = Curve.BooleanKeyValues.Emplace();

		KeyTimes.Reserve(UsdTimeSamples.Num());
		BooleanKeyValues.Reserve(UsdTimeSamples.Num());

		const FFrameRate StageFrameRate{static_cast<uint32>(UsdStage.GetTimeCodesPerSecond()), 1};

		double LastTimeSample = TNumericLimits<double>::Lowest();
		for (const double UsdTimeSample : UsdTimeSamples)
		{
			// We never want to evaluate the same time twice
			if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
			{
				continue;
			}
			LastTimeSample = UsdTimeSample;

			int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
			float SubFrameNumber = UsdTimeSample - FrameNumber;

			FFrameTime FrameTime{FrameNumber, SubFrameNumber};
			double FrameTimeSeconds = static_cast<float>(StageFrameRate.AsSeconds(FrameTime));

			bool UEValue = ReaderFunc(UsdTimeSample);

			KeyTimes.Add(FrameTimeSeconds);
			BooleanKeyValues.Add(UEValue);
		}

		return true;
	}

	bool ReadFloats(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<float(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		OutPayloadData.Curves.SetNum(1);
		FRichCurve& Curve = OutPayloadData.Curves[0];

		const FFrameRate StageFrameRate{static_cast<uint32>(UsdStage.GetTimeCodesPerSecond()), 1};
		const ERichCurveInterpMode InterpMode = (UsdStage.GetInterpolationType() == EUsdInterpolationType::Linear)
													? ERichCurveInterpMode::RCIM_Linear
													: ERichCurveInterpMode::RCIM_Constant;

		double LastTimeSample = TNumericLimits<double>::Lowest();
		for (const double UsdTimeSample : UsdTimeSamples)
		{
			// We never want to evaluate the same time twice
			if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
			{
				continue;
			}
			LastTimeSample = UsdTimeSample;

			int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
			float SubFrameNumber = UsdTimeSample - FrameNumber;

			FFrameTime FrameTime{FrameNumber, SubFrameNumber};
			double FrameTimeSeconds = static_cast<float>(StageFrameRate.AsSeconds(FrameTime));

			float UEValue = ReaderFunc(UsdTimeSample);

			FKeyHandle Handle = Curve.AddKey(FrameTimeSeconds, UEValue);
			Curve.SetKeyInterpMode(Handle, InterpMode);
		}

		return true;
	}

	bool ReadColors(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FLinearColor(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		OutPayloadData.Curves.SetNum(4);
		FRichCurve& RCurve = OutPayloadData.Curves[0];
		FRichCurve& GCurve = OutPayloadData.Curves[1];
		FRichCurve& BCurve = OutPayloadData.Curves[2];
		FRichCurve& ACurve = OutPayloadData.Curves[3];

		const FFrameRate StageFrameRate{static_cast<uint32>(UsdStage.GetTimeCodesPerSecond()), 1};
		const ERichCurveInterpMode InterpMode = (UsdStage.GetInterpolationType() == EUsdInterpolationType::Linear)
													? ERichCurveInterpMode::RCIM_Linear
													: ERichCurveInterpMode::RCIM_Constant;

		double LastTimeSample = TNumericLimits<double>::Lowest();
		for (const double UsdTimeSample : UsdTimeSamples)
		{
			// We never want to evaluate the same time twice
			if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
			{
				continue;
			}
			LastTimeSample = UsdTimeSample;

			int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
			float SubFrameNumber = UsdTimeSample - FrameNumber;

			FFrameTime FrameTime{FrameNumber, SubFrameNumber};
			double FrameTimeSeconds = static_cast<float>(StageFrameRate.AsSeconds(FrameTime));

			FLinearColor UEValue = ReaderFunc(UsdTimeSample);

			FKeyHandle RHandle = RCurve.AddKey(FrameTimeSeconds, UEValue.R);
			FKeyHandle GHandle = GCurve.AddKey(FrameTimeSeconds, UEValue.G);
			FKeyHandle BHandle = BCurve.AddKey(FrameTimeSeconds, UEValue.B);
			FKeyHandle AHandle = ACurve.AddKey(FrameTimeSeconds, UEValue.A);

			RCurve.SetKeyInterpMode(RHandle, InterpMode);
			GCurve.SetKeyInterpMode(GHandle, InterpMode);
			BCurve.SetKeyInterpMode(BHandle, InterpMode);
			ACurve.SetKeyInterpMode(AHandle, InterpMode);
		}

		return true;
	}

	bool ReadTransforms(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FTransform(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		OutPayloadData.Curves.SetNum(9);
		FRichCurve& TransXCurve = OutPayloadData.Curves[0];
		FRichCurve& TransYCurve = OutPayloadData.Curves[1];
		FRichCurve& TransZCurve = OutPayloadData.Curves[2];
		FRichCurve& RotXCurve = OutPayloadData.Curves[3];
		FRichCurve& RotYCurve = OutPayloadData.Curves[4];
		FRichCurve& RotZCurve = OutPayloadData.Curves[5];
		FRichCurve& ScaleXCurve = OutPayloadData.Curves[6];
		FRichCurve& ScaleYCurve = OutPayloadData.Curves[7];
		FRichCurve& ScaleZCurve = OutPayloadData.Curves[8];

		const FFrameRate StageFrameRate{static_cast<uint32>(UsdStage.GetTimeCodesPerSecond()), 1};
		const ERichCurveInterpMode InterpMode = (UsdStage.GetInterpolationType() == EUsdInterpolationType::Linear)
													? ERichCurveInterpMode::RCIM_Linear
													: ERichCurveInterpMode::RCIM_Constant;
		struct FHeadingHelper
		{
			FRotator Heading;
			bool bHeadingSet = false;
		};
		FHeadingHelper HeadingHelper;

		double LastTimeSample = TNumericLimits<double>::Lowest();
		for (const double UsdTimeSample : UsdTimeSamples)
		{
			// We never want to evaluate the same time twice
			if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
			{
				continue;
			}
			LastTimeSample = UsdTimeSample;

			int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
			float SubFrameNumber = UsdTimeSample - FrameNumber;

			FFrameTime FrameTime{FrameNumber, SubFrameNumber};
			double FrameTimeSeconds = static_cast<float>(StageFrameRate.AsSeconds(FrameTime));

			FTransform UEValue = ReaderFunc(UsdTimeSample);
			FVector Location = UEValue.GetLocation();
			FRotator Rotator = UEValue.Rotator();
			FVector Scale = UEValue.GetScale3D();

			if (HeadingHelper.bHeadingSet)
			{
				FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Roll, Rotator.Roll);
				FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Pitch, Rotator.Pitch);
				FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Yaw, Rotator.Yaw);

				FRotator OtherChoice = Rotator.GetEquivalentRotator().GetNormalized();
				float FirstDiff = HeadingHelper.Heading.GetManhattanDistance(Rotator);
				float SecondDiff = HeadingHelper.Heading.GetManhattanDistance(OtherChoice);
				if (SecondDiff < FirstDiff)
				{
					Rotator = OtherChoice;
				}
			}
			else
			{
				HeadingHelper.bHeadingSet = true;
			}

			HeadingHelper.Heading = Rotator;

			FKeyHandle HandleTransX = TransXCurve.AddKey(FrameTimeSeconds, Location.X);
			FKeyHandle HandleTransY = TransYCurve.AddKey(FrameTimeSeconds, Location.Y);
			FKeyHandle HandleTransZ = TransZCurve.AddKey(FrameTimeSeconds, Location.Z);
			FKeyHandle HandleRotX = RotXCurve.AddKey(FrameTimeSeconds, Rotator.Roll);
			FKeyHandle HandleRotY = RotYCurve.AddKey(FrameTimeSeconds, Rotator.Pitch);
			FKeyHandle HandleRotZ = RotZCurve.AddKey(FrameTimeSeconds, Rotator.Yaw);
			FKeyHandle HandleScaleX = ScaleXCurve.AddKey(FrameTimeSeconds, Scale.X);
			FKeyHandle HandleScaleY = ScaleYCurve.AddKey(FrameTimeSeconds, Scale.Y);
			FKeyHandle HandleScaleZ = ScaleZCurve.AddKey(FrameTimeSeconds, Scale.Z);

			TransXCurve.SetKeyInterpMode(HandleTransX, InterpMode);
			TransYCurve.SetKeyInterpMode(HandleTransY, InterpMode);
			TransZCurve.SetKeyInterpMode(HandleTransZ, InterpMode);
			RotXCurve.SetKeyInterpMode(HandleRotX, InterpMode);
			RotYCurve.SetKeyInterpMode(HandleRotY, InterpMode);
			RotZCurve.SetKeyInterpMode(HandleRotZ, InterpMode);
			ScaleXCurve.SetKeyInterpMode(HandleScaleX, InterpMode);
			ScaleYCurve.SetKeyInterpMode(HandleScaleY, InterpMode);
			ScaleZCurve.SetKeyInterpMode(HandleScaleZ, InterpMode);
		}

		return true;
	}

	bool ReadRawTransforms(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FTransform(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		OutPayloadData.Transforms.Reserve(UsdTimeSamples.Num());

		double LastTimeSample = TNumericLimits<double>::Lowest();
		for (const double UsdTimeSample : UsdTimeSamples)
		{
			// We never want to evaluate the same time twice
			if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
			{
				continue;
			}
			LastTimeSample = UsdTimeSample;

			OutPayloadData.Transforms.Emplace(ReaderFunc(UsdTimeSample));
		}

		return true;
	}

	// Sets on the provided MeshNode custom attributes needed to bake the provided GeomPropValue geomprops / primvars
	// into textures, for the provided MaterialNodeUid
	void AddPrimvarBakingAttributes(
		UInterchangeBaseNode* BaseNode,
		const FString& MaterialNodeUid,
		UInterchangeBaseNodeContainer& NodeContainer,
		const TArray<FUsdMaterialXShaderGraph::FGeomProp>& GeomPropValues
	)
	{
#if WITH_EDITOR
		using namespace UE::Interchange;

		// TODO: Is it OK that this function is called repeatedly for the same Mesh? (e.g. from multiple material assignments,
		// from multiple scene node instancing the same Mesh asset node, etc.)

		UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode);
		if (!MeshNode)
		{
			// We also use this function when setting material overrides on scene nodes. In that case our scene node should have a
			// Mesh asset node UID
			if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
			{
				FString AssetInstanceUid;
				SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid);

				if (const UInterchangeMeshNode* SceneMeshNode = Cast<UInterchangeMeshNode>(NodeContainer.GetNode(AssetInstanceUid)))
				{
					MeshNode = const_cast<UInterchangeMeshNode*>(SceneMeshNode);
				}
			}
		}
		if (!MeshNode)
		{
			return;
		}

		// Walk the shader graph rooted at the material node and build a geomprop-name -> shader-UID
		// map, so each GeomPropValues entry below can be paired with its TextureSample shader node
		// by *name* rather than by traversal order. The MaterialX translator stamps the geomprop
		// name on its TextureSample shader nodes via the UE:GeomPropName attribute (set in
		// USDMaterialXShaderGraph.cpp + carried through in MaterialXSurfaceShaderAbstract.cpp).
		//
		// Reachability via connection walking (rather than filtering by "UID starts with the
		// material UID") avoids assumptions about how the schema handlers mint their UIDs --
		// notably MaterialXSchemaHandler's clones produce per-target UIDs that don't share the
		// material UID as a textual prefix. We follow both Inputs:*:Connect connections and
		// string-valued Inputs:*:Value / :Parameter inputs, mirroring the deep-clone walk in
		// MaterialXSchemaHandler.
		TMap<FString, FString> GeomPropNameToShaderUid;
		if (const UInterchangeShaderNode* RootMaterialNode = Cast<UInterchangeShaderNode>(NodeContainer.GetNode(MaterialNodeUid)))
		{
			using namespace UE::Interchange::Materials::Standard::Nodes;

			// Temporarily copy-pasted here so that we can submit this on a patch version
			const static FString GeomPropNameAttribute = TEXT("UE:GeomPropName");

			TSet<FString> Visited;
			TArray<const UInterchangeShaderNode*> ToProcess;

			Visited.Add(MaterialNodeUid);
			ToProcess.Add(RootMaterialNode);

			while (!ToProcess.IsEmpty())
			{
				const UInterchangeShaderNode* Current = ToProcess.Pop();

				FString ShaderType;
				bool bIsGeomProp = false;
				FString GeomPropName;
				if (Current->GetCustomShaderType(ShaderType)
					&& ShaderType == TextureSample::Name.ToString()
					&& Current->GetBooleanAttribute(::MaterialX::Attributes::GeomPropImage, bIsGeomProp)
					&& bIsGeomProp
					&& Current->GetStringAttribute(GeomPropNameAttribute, GeomPropName)
					&& !GeomPropName.IsEmpty())
				{
					GeomPropNameToShaderUid.Add(GeomPropName, Current->GetUniqueID());
				}

				TArray<FString> InputNames;
				UInterchangeShaderPortsAPI::GatherInputs(Current, InputNames);
				for (const FString& InputName : InputNames)
				{
					FString ConnectedUid;
					FString OutputName;
					if (UInterchangeShaderPortsAPI::GetInputConnection(Current, InputName, ConnectedUid, OutputName) && !ConnectedUid.IsEmpty())
					{
						bool bAlreadyVisited = false;
						Visited.Add(ConnectedUid, &bAlreadyVisited);
						if (!bAlreadyVisited)
						{
							if (const UInterchangeShaderNode* ConnectedShader = Cast<UInterchangeShaderNode>(NodeContainer.GetNode(ConnectedUid)))
							{
								ToProcess.Add(ConnectedShader);
							}
						}
					}
				}
			}
		}

		if (GeomPropValues.Num() != GeomPropNameToShaderUid.Num())
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"FailedToBakePrimvars",
					"Failed to bake primvars for mesh '{0}' and material '{1}': Encountered {2} GeomPropValues but {3} geomprop shader nodes!"
				),
				FText::FromString(MeshNode->GetUniqueID()),
				FText::FromString(MaterialNodeUid),
				GeomPropValues.Num(),
				GeomPropNameToShaderUid.Num()
			));

			return;
		}

		MeshNode->AddInt32Attribute(USD::Primvar::Number, GeomPropValues.Num());

		for (int32 Index = 0; Index < GeomPropValues.Num(); ++Index)
		{
			const FString* ShaderUidForGeomProp = GeomPropNameToShaderUid.Find(GeomPropValues[Index].Name);
			if (!ShaderUidForGeomProp)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"FailedToBakePrimvars_MissingGeomPropShader",
						"Failed to bake primvars for mesh '{0}' and material '{1}': No geomprop TextureSample shader node found for primvar '{2}'."
					),
					FText::FromString(MeshNode->GetUniqueID()),
					FText::FromString(MaterialNodeUid),
					FText::FromString(GeomPropValues[Index].Name)
				));

				return;
			}

			MeshNode->AddStringAttribute(USD::Primvar::Name + FString::FromInt(Index), GeomPropValues[Index].Name);
			MeshNode->AddBooleanAttribute(USD::Primvar::TangentSpace + FString::FromInt(Index), GeomPropValues[Index].bTangentSpace);
			MeshNode->AddStringAttribute(USD::Primvar::ShaderNodeTextureSample + FString::FromInt(Index), *ShaderUidForGeomProp);
		}
#endif	  // WITH_EDITOR
	}

	UInterchangeBaseNode* GetOrCreateTwoSidedMaterial(UInterchangeBaseNode* MaterialNode, UInterchangeBaseNodeContainer& NodeContainer)
	{
		using namespace UsdUnreal::MaterialUtils;

		if (!MaterialNode)
		{
			return nullptr;
		}

		if (UInterchangeMaterialReferenceNode* ReferenceNode = Cast<UInterchangeMaterialReferenceNode>(MaterialNode))
		{
			// Can't do much here, this is a reference to an existing UAsset
			return ReferenceNode;
		}

		if (const UInterchangeShaderGraphNode* OneSidedNode = Cast<UInterchangeShaderGraphNode>(MaterialNode))
		{
			// "One-sided" node is already two-sided, so just return that.
			// This can happen e.g. if the MaterialX translator internally sets the node as two-sided because the shader graph says to do that.
			// Note that we don't have anything caring for the exact opposite: If the USD Mesh is explicitly one-sided and the MaterialX material is
			// two-sided in this way we'll just use the two-sided material on the mesh. For now let's presume that's what the user intended, as you
			// have to explicitly set the MaterialX material as two-sided for that
			bool bIsTwoSided = false;
			if (OneSidedNode->GetCustomTwoSided(bIsTwoSided) && bIsTwoSided)
			{
				return MaterialNode;
			}
		}

		const FString TwoSidedUid = MaterialNode->GetUniqueID() + TwoSidedSuffix;
		const FString TwoSidedNodeName = MaterialNode->GetDisplayLabel() + TwoSidedSuffix;

		// We already created this two-sided node, just return it
		if (const UInterchangeBaseNode* BaseNode = NodeContainer.GetNode(TwoSidedUid))
		{
			ensure(BaseNode->IsA<UInterchangeMaterialInstanceNode>() || BaseNode->IsA<UInterchangeShaderGraphNode>());
			return const_cast<UInterchangeBaseNode*>(BaseNode);
		}

		// If it's a regular usdpreviewsurface material, just create a new node using the two-sided version as its parent
		if (UInterchangeMaterialInstanceNode* InstanceNode = Cast<UInterchangeMaterialInstanceNode>(MaterialNode))
		{
			FString CustomParentString;
			if (InstanceNode->GetCustomParent(CustomParentString))
			{
				FSoftObjectPath ParentMaterialPath{CustomParentString};

				if (ParentMaterialPath.IsValid() && IsReferencePreviewSurfaceMaterial(ParentMaterialPath))
				{
					FSoftObjectPath TwoSidedParent = GetTwoSidedVersionOfReferencePreviewSurfaceMaterial(ParentMaterialPath);
					if (TwoSidedParent == ParentMaterialPath)
					{
						// PreviewSurface material instance is already two sided
						return InstanceNode;
					}

					if (UInterchangeMaterialInstanceNode* ExistingTwoSided = GetExistingNode<UInterchangeMaterialInstanceNode>(NodeContainer, TwoSidedUid))
					{
						return ExistingTwoSided;
					}

					UInterchangeMaterialInstanceNode* TwoSidedNode = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
					UInterchangeBaseNode::CopyStorage(InstanceNode, TwoSidedNode);
					NodeContainer.SetupNode(TwoSidedNode, TwoSidedUid, TwoSidedNodeName, EInterchangeNodeContainerType::TranslatedAsset);
					TwoSidedNode->SetCustomParent(TwoSidedParent.ToString());
					return TwoSidedNode;
				}
			}
		}

		// Otherwise create a material instance of whatever material node we got, and set it as two-sided
		UInterchangeMaterialInstanceNode* TwoSidedNode = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
		NodeContainer.SetupNode(TwoSidedNode, TwoSidedUid, TwoSidedNodeName, EInterchangeNodeContainerType::TranslatedAsset);
		TwoSidedNode->SetCustomParent(MaterialNode->GetUniqueID());
		TwoSidedNode->AddBooleanAttribute(UsdMaterialInstanceFromShaderGraph, true);
		TwoSidedNode->SetCustomTwoSided(true);
		return TwoSidedNode;
	}

	UInterchangeBaseNode* TranslateMaterialNode(
		const UE::FUsdPrim& MaterialPrim,
		UInterchangeUsdContext& UsdContext,
		FHandlerAccumulatedInfo& AccumulatedInfo
	)
	{
		using namespace UE::Interchange::USD;

		if (UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator())
		{
			FTraversalInfo TraversalInfo;
			if (TOptional<FHandlerAccumulatedInfo> AccumulatedInfoForPrim = CurrentTranslator->TranslatePrim(MaterialPrim, TraversalInfo))
			{
				// Do this so we pick up any texture nodes or anything else
				AccumulatedInfo.AppendInfo(AccumulatedInfoForPrim.GetValue());

				// Just return the main asset node for now (the material node classes don't really share a common base...)
				if (AccumulatedInfoForPrim->PrimAssetNodes.Num() > 0)
				{
					return AccumulatedInfoForPrim->PrimAssetNodes[0];
				}
			}
		}

		return nullptr;
	}

	template<typename MeshOrSceneNodeType>
	void SetSlotMaterialDependencies(
		MeshOrSceneNodeType* MeshOrSceneNode,
		const UsdUtils::FUsdPrimMaterialAssignmentInfo& MaterialAssignments,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		using namespace USDSchemaHandlerUtils::Private;

		TSet<FString> UsedSlotNames;

		for (const UsdUtils::FUsdPrimMaterialSlot& Slot : MaterialAssignments.Slots)
		{
			// We do this because Interchange will, in some scenarios, merge material slots with identical slot names.
			// By using the source (which is the displaycolor desc / material prim path / unreal material content path)
			// we do end up with a goofy looking super long material slot names, but it will have Interchange only combine
			// slots if they really are pointing at the exact same thing
			FString SlotName = Slot.MaterialSource;

			// Get the Uid of the material instance that we'll end up assigning to this slot
			FString MaterialUid;
			switch (Slot.AssignmentType)
			{
				case UsdUtils::EPrimAssignmentType::DisplayColor:
				{
					// MaterialSource here is e.g. "!DisplayColor_0_1"
					// We use the parent material content path directly as the "MaterialUid" here instead
					// of creating a material instance node. The factory resolution code will fall back to
					// treating it as a content path when it fails to find a node with this UID
					using namespace UsdUnreal::MaterialUtils;
					TOptional<FDisplayColorMaterial> ParsedMat = FDisplayColorMaterial::FromString(Slot.MaterialSource);
					if (ParsedMat)
					{
						if (const FSoftObjectPath* Path = GetReferenceMaterialPath(ParsedMat.GetValue()))
						{
							MaterialUid = Path->GetAssetPathString();
						}
					}
					break;
				}
				case UsdUtils::EPrimAssignmentType::MaterialPrim:
				{
					// MaterialSource here is the material prim path
					UE::FUsdPrim MaterialPrim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*Slot.MaterialSource});

					// Go through a prototype in case this material is inside an instance proxy, as this needs to match up with
					// the polygon group name we're going to add to our mesh description.
					UE::FUsdPrim MaterialPrototypePrim = UsdUtils::GetPrototypePrim(MaterialPrim);
					if (MaterialPrototypePrim != MaterialPrim)
					{
						SlotName = MaterialPrototypePrim.GetPrimPath().GetString();
					}

					if (UInterchangeBaseNode* MaterialNode = TranslateMaterialNode(MaterialPrim, UsdContext, AccumulatedInfo))
					{
						// TODO: Revamp material slot name handling
						//
						// The main priority is to make sure the slot name we set on the mesh node slot dependency matches what we end up
						// putting on the mesh description slot names when producing payload data (from FixMaterialSlotNames()).
						// 
						// When we're talking about instancing, we still need to make sure both use the same slot name, but now with Pregen 
						// handling instancing deduplication (and turning off InstancingAwareTranslation) we have no choice
						// but to "wash" the slot name through Pregen's as well. Otherwise, we may end up with broken slot dependencies for mesh 
						// instances, as pregen will deduplicate the mesh and materials themselves, but the slot name on the node's slot 
						// dependency may point at one of the instances, and the slot name on the mesh description may point at another... 
						SlotName = UsdContext.MakeAssetNodeUid(MaterialPrim, MaterialPrefix);
						
						MaterialUid = MaterialNode->GetUniqueID();

						if (Slot.bMeshIsDoubleSided)
						{
							if (UInterchangeBaseNode* TwoSidedMaterial = GetOrCreateTwoSidedMaterial(MaterialNode, *UsdContext.GetNodeContainer()))
							{
								AccumulatedInfo.PrimAssetNodes.Add(TwoSidedMaterial);
								MaterialUid = TwoSidedMaterial->GetUniqueID();
							}
						}
					}

					if (TArray<FUsdMaterialXShaderGraph::FGeomProp>* FoundGeomProps = UsdContext.MaterialUidToGeomProps.Find(MaterialUid))
					{
						AddPrimvarBakingAttributes(MeshOrSceneNode, MaterialUid, *UsdContext.GetNodeContainer(), *FoundGeomProps);
					}
					break;
				}
				case UsdUtils::EPrimAssignmentType::UnrealMaterial:
				{
					// MaterialSource here is the content path, e.g. "/Game/MyFolder/Red.Red"
					MaterialUid = AddUnrealMaterialReferenceNodeIfNeeded(Slot.MaterialSource, *UsdContext.GetNodeContainer(), AccumulatedInfo);
					break;
				}
				default:
				{
					ensure(false);
					break;
				}
			}

			// It's possible to have multiple mesh sections with the same material bindings within a mesh node so make sure the
			// slot names are unique to keep them separate
			FString UniqueSlotName = UsdUnreal::ObjectUtils::GetUniqueName(SlotName, UsedSlotNames);
			UsedSlotNames.Add(UniqueSlotName);

			MeshOrSceneNode->SetSlotMaterialDependencyUid(UniqueSlotName, *MaterialUid);
		}
	}
	template void SetSlotMaterialDependencies(UInterchangeMeshNode*, const UsdUtils::FUsdPrimMaterialAssignmentInfo&, FHandlerAccumulatedInfo&, UInterchangeUsdContext&);
	template void SetSlotMaterialDependencies(UInterchangeSceneNode*, const UsdUtils::FUsdPrimMaterialAssignmentInfo&, FHandlerAccumulatedInfo&, UInterchangeUsdContext&);

	void FixMaterialSlotNames(
		FMeshDescription& MeshDescription,
		const TArray<UsdUtils::FUsdPrimMaterialSlot>& MeshAssingmentSlots,
		UInterchangeUsdContext& UsdContext
	)
	{
		UE::FUsdStage Stage = UsdContext.GetUsdStage();
		if (!Stage)
		{
			return;
		}

		FStaticMeshAttributes StaticMeshAttributes(MeshDescription);
		for (int32 MaterialSlotIndex = 0; MaterialSlotIndex < StaticMeshAttributes.GetPolygonGroupMaterialSlotNames().GetNumElements();
			 ++MaterialSlotIndex)
		{
			int32 MaterialIndex = 0;
			LexFromString(MaterialIndex, *StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex].ToString());

			if (MeshAssingmentSlots.IsValidIndex(MaterialIndex))
			{
				FString SlotName = MeshAssingmentSlots[MaterialIndex].MaterialSource;
				
				// The USDCore mesh conversion functions should have put the material prim path as the slot name for
				// slots meant to be connected to the materials produced from prims (as opposed to e.g. displayColor
				// or UnrealMaterials)
				if (UE::FSdfPath::IsValidPathString(*SlotName))
				{
					if (UE::FUsdPrim MaterialPrim = Stage.GetPrimAtPath(UE::FSdfPath{*SlotName}))
					{
						// See the huge comment inside the MaterialPrim case in SetSlotMaterialDependencies()
						SlotName = UsdContext.MakeAssetNodeUid(MaterialPrim, MaterialPrefix);
					}
				}

				StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex] = *SlotName;
			}
		}
	}

	// Deprecated
	void FixMaterialSlotNames(FMeshDescription& MeshDescription, const TArray<UsdUtils::FUsdPrimMaterialSlot>& MeshAssingmentSlots)
	{
		FStaticMeshAttributes StaticMeshAttributes(MeshDescription);
		for (int32 MaterialSlotIndex = 0; MaterialSlotIndex < StaticMeshAttributes.GetPolygonGroupMaterialSlotNames().GetNumElements();
			 ++MaterialSlotIndex)
		{
			int32 MaterialIndex = 0;
			LexFromString(MaterialIndex, *StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex].ToString());

			if (MeshAssingmentSlots.IsValidIndex(MaterialIndex))
			{
				const FString& SlotName = MeshAssingmentSlots[MaterialIndex].MaterialSource;
				StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex] = *SlotName;
			}
		}
	}

	bool IsValidLODName(const FString& PrimName)
	{
		return GetLODIndexFromName(PrimName) != INDEX_NONE;
	}

	int32 GetLODIndexFromName(FString Name)
	{
		//Tries to remove LOD_ first, if that does not work, removes LOD only (LODN pattern handling)
		const static FString LOD_String = LODString + TEXT("_");
		if (!Name.RemoveFromStart(LOD_String) && !Name.RemoveFromStart(LODString))
		{
			return INDEX_NONE;
		}

		if (!Name.IsNumeric())
		{
			return INDEX_NONE;
		}

		int32 Index = INDEX_NONE;
		LexFromString(Index, *Name);
		return Index;
	}

	UE::FUsdPrim GetLODMesh(const UE::FUsdPrim& LODContainerPrim, const FString& LODName)
	{
		UE::FUsdPrim LODPrim = GetLODPrim(LODContainerPrim, LODName);
		if (LODPrim && LODPrim.IsA(TEXT("Mesh")))
		{
			return LODPrim;
		}
		
		return {};
	};

	UE::FUsdPrim GetLODPrim(const UE::FUsdPrim& LODContainerPrim, const FString& LODName)
	{
		const UE::FSdfPath IdealPrimPath = LODContainerPrim.GetPrimPath().AppendChild(*LODName);

		UE::FUsdPrim Prim = LODContainerPrim.GetStage().GetPrimAtPath(IdealPrimPath);
		if (Prim && Prim.IsActive())
		{
			return Prim;
		}

		return {};
	};

	UE::FUsdPrim TryGettingInactiveLODPrim(const FString& PrimPathString, UInterchangeUsdContext& UsdContext)
	{
		if (PrimPathString.IsEmpty())
		{
			return {};
		}

		UE::FSdfPath PrimPath{ *PrimPathString };

		//Check if any of the prims in the hierarchy matches a prim name:
		//Note: this is needed so we can handle nested hierarchies inside LOD variants.
		FString LODContainerPathString;
		bool bFoundValidLODName = false;
		FString VariantName;
		UE::FSdfPath CurrentPrimPath = PrimPath;
		while (!CurrentPrimPath.IsEmpty())
		{
			FString CurrentPrimName = CurrentPrimPath.GetName();
			UE::FSdfPath ParentPath = CurrentPrimPath.GetParentPath();

			if (IsValidLODName(CurrentPrimName))
			{
				VariantName = CurrentPrimName;
				bFoundValidLODName = true;
				LODContainerPathString = ParentPath.GetString();
				break;
			}
			CurrentPrimPath = ParentPath;
		}

		if (!bFoundValidLODName)
		{
			return {};
		}

		const UE::FSdfPath LODContainerPath{*LODContainerPathString};

		if (LODContainerPath.IsEmpty())
		{
			return {};
		}

		UE::FUsdStage TempStage;

		{
			FWriteScopeLock Lock{UsdContext.PrimPathToVariantToStageLock};

			// Check if we have the stage we want already
			if (TMap<FString, UE::FUsdStage>* TempStagesForPrim = UsdContext.PrimPathToVariantToStage.Find(LODContainerPathString))
			{
				if (UE::FUsdStage* TempStagesForVariant = TempStagesForPrim->Find(VariantName))
				{
					TempStage = *TempStagesForVariant;
				}
			}

			// Open a brand new stage we can freely flip our variants in without invalidating other prim references from concurrent tasks.
			// We won't use the stage cache for these, so our strong reference right here is the only thing holding the stage opened
			if (!TempStage)
			{
				// We used to use a population mask here to only compose the prim subtree of the LOD prim we want to parse, but
				// that is too prone to edge case issues as any relationship to prims outside the loaded subtree would fail and 
				// emit errors (blend shapes, material assignments, etc.)
				const bool bUseStageCache = false;
				TempStage = UnrealUSDWrapper::OpenStage(
					*UsdContext.GetUsdStage().GetRootLayer().GetIdentifier(),
					EUsdInitialLoadSet::LoadAll,
					bUseStageCache
				);

				if (!TempStage)
				{
					return {};
				}

				UE::FUsdPrim LODContainerPrim = TempStage.GetPrimAtPath(LODContainerPath);
				if (!LODContainerPrim)
				{
					return {};
				}

				// We have to edit the session layer here, and not the root layer directly. This because USD only opens a layer
				// once in memory, so if we have multiple of these temp stages all trying to set the variant to different values
				// on the root layer itself, they'd be actually trying to overwrite each other and could even lead to threading issues.
				//
				// The session layer however is unique to each of these temp stages so we won't have that problem, and it still
				// should compose the variant switch just the same.
				UE::FUsdEditContext EditContext{TempStage, TempStage.GetSessionLayer()};

				UE::FUsdVariantSets VariantSets = LODContainerPrim.GetVariantSets();
				bool bSwitched = VariantSets.SetSelection(UE::Interchange::USD::LODString, VariantName);
				if (!bSwitched)
				{
					return {};
				}

				// We finally have our stage for the particular lod container and with the variant we want: Cache it for later, if needed.
				UsdContext.PrimPathToVariantToStage.FindOrAdd(LODContainerPathString).Add(VariantName, TempStage);
			}
		}

		return TempStage.GetPrimAtPath(PrimPath);
	}

	bool CheckAndChopPayloadPrefix(FString& PayloadKey, const FString& Prefix)
	{
		if (PayloadKey.StartsWith(Prefix))
		{
			// Remove the prefix from the payload key
			PayloadKey = PayloadKey.RightChop(Prefix.Len());

			return true;
		}

		return false;
	}

	bool DecodeTexturePayloadKey(const FString& PayloadKey, FString& OutTextureFilePath, TextureGroup& OutTextureGroup)
	{
		// Use split from end here so that we ignore any backslashes within the file path itself
		FString FilePath;
		FString TextureGroupStr;
		bool bSplit = PayloadKey.Split(TEXT("\\"), &FilePath, &TextureGroupStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!bSplit)
		{
			return false;
		}

		OutTextureFilePath = FilePath;

		int32 TempInt;
		if (LexTryParseString<int32>(TempInt, *TextureGroupStr))
		{
			OutTextureGroup = (TextureGroup)TempInt;
		}

		return true;
	}

	FString EncodeTexturePayloadKey(const UsdToUnreal::FTextureParameterValue& Value)
	{
		// Encode the compression settings onto the payload key as we need to move that into the
		// payload data within UInterchangeUSDTranslator::GetTexturePayloadData.
		//
		// This should be a temporary thing, and in the future we'll be able to store compression
		// settings directly on the texture translated node
		return Value.TextureFilePath + TEXT("\\") + LexToString((int32)Value.Group);
	}

	FString GetMorphTargetCurvePayloadKey(const FString& SkeletonPrimPath, int32 SkelAnimChannelIndex, const FString& BlendShapePath)
	{
		return FString::Printf(TEXT("%s\\%d\\%s"), *SkeletonPrimPath, SkelAnimChannelIndex, *BlendShapePath);
	}

	FString GetMorphTargetMeshPayloadKey(
		bool bIsInsideLOD,
		const FString& MeshPrimPath,
		int32 MeshBlendShapeIndex,
		const FString& InbetweenName
	)
	{
		return FString::Printf(TEXT("%s%s\\%d\\%s"), bIsInsideLOD ? *LODPrefix : TEXT(""), *MeshPrimPath, MeshBlendShapeIndex, *InbetweenName);
	}

	// TODO: Cleanup/unify/standardize these payload manipulating functions (don't add/remove prefixes everywhere but have a standard format, etc.)
	bool ParseMorphTargetMeshPayloadKey(
		FString InPayloadKey,
		bool& bOutIsLODMesh,
		FString& OutMeshPrimPath,
		int32& OutBlendShapeIndex,
		FString& OutInbetweenName
	)
	{
		bool bIsLODMesh = CheckAndChopPayloadPrefix(InPayloadKey, LODPrefix);

		// These payload keys are generated by GetMorphTargetMeshPayloadKey(), and so should take the form
		// "<mesh prim path>\<mesh blend shape index>\<optional inbetween name>"
		const bool bCullEmpty = false;
		TArray<FString> PayloadKeyTokens;
		InPayloadKey.ParseIntoArray(PayloadKeyTokens, TEXT("\\"), bCullEmpty);
		if (PayloadKeyTokens.Num() != 3)
		{
			return false;
		}

		const FString& MeshPrimPath = PayloadKeyTokens[0];
		const FString& BlendShapeIndexStr = PayloadKeyTokens[1];
		const FString& InbetweenName = PayloadKeyTokens[2];

		int32 BlendShapeIndex = INDEX_NONE;
		bool bLexed = LexTryParseString(BlendShapeIndex, *BlendShapeIndexStr);
		if (!bLexed)
		{
			return false;
		}

		bOutIsLODMesh = bIsLODMesh;
		OutMeshPrimPath = MeshPrimPath;
		OutBlendShapeIndex = BlendShapeIndex;
		OutInbetweenName = InbetweenName;
		return true;
	}

	FString HashAnimPayloadQuery(const Interchange::FAnimationPayloadQuery& Query)
	{
		FSHAHash Hash;
		FSHA1 SHA1;

		// TODO: Is there a StringView alternative?
		FString SkeletonPrimPath;
		FString JointIndexStr;
		bool bSplit = Query.PayloadKey.UniqueId.Split(TEXT("\\"), &SkeletonPrimPath, &JointIndexStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!bSplit)
		{
			return {};
		}

		SHA1.UpdateWithString(*SkeletonPrimPath, SkeletonPrimPath.Len());

		SHA1.Update(reinterpret_cast<const uint8*>(&Query.TimeDescription.BakeFrequency), sizeof(Query.TimeDescription.BakeFrequency));
		SHA1.Update(reinterpret_cast<const uint8*>(&Query.TimeDescription.RangeStartSecond), sizeof(Query.TimeDescription.RangeStartSecond));
		SHA1.Update(reinterpret_cast<const uint8*>(&Query.TimeDescription.RangeStopSecond), sizeof(Query.TimeDescription.RangeStopSecond));

		SHA1.Final();
		SHA1.GetHash(&Hash.Hash[0]);
		return Hash.ToString();
	}

	TArray<UE::Interchange::FNaniteAssemblyDescription::FBoneInfluence> CreateBoneInfluences(
		const TArray<int32>& JointIndices, 
		const TArray<float>& JointWeights)
	{
		// Create the bone influence (joint index + weight) array from the primvar attributes.
		TArray<UE::Interchange::FNaniteAssemblyDescription::FBoneInfluence> BoneInfluences;
		if (ensure(JointIndices.Num() == JointWeights.Num()))
		{
			BoneInfluences.Reserve(JointIndices.Num());
			for (int32 InfluenceIndex = 0; InfluenceIndex < JointIndices.Num(); ++InfluenceIndex)
			{
				BoneInfluences.Emplace(JointIndices[InfluenceIndex], JointWeights[InfluenceIndex]);
			}
		}

		return BoneInfluences;
	}

	void GetNaniteAssemblyPayloadDataForPrims(
		const UE::FUsdStage& UsdStage,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult& TraversalResult,
		const FString& SkelIdentifier,
		const TArray<FString>& JointNames,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription)
	{
		using namespace UsdToUnreal::NaniteAssemblyUtils;

		const bool bIsNewDescription = !OutNaniteAssemblyDescription.IsSet();
		const int32 PartIndexStartOffset = bIsNewDescription ? 0 : OutNaniteAssemblyDescription.GetValue().PartUids.Num();

		TArray<TSharedPtr<FMeshEntry>> NativeInstancedEntries;
		TMap<FString, int32> MeshUidToPartIndexTable;
		TArray<TArray<FString>> PartUids;

		if (!TraversalResult.CheckAndGetNativeInstancedPartMeshEntries(NativeInstancedEntries, MeshUidToPartIndexTable, PartUids)
			|| NativeInstancedEntries.IsEmpty())
		{
			return;
		}

		if (bIsNewDescription)
		{
			OutNaniteAssemblyDescription = UE::Interchange::FNaniteAssemblyDescription{};
		}
		UE::Interchange::FNaniteAssemblyDescription& Description = OutNaniteAssemblyDescription.GetValue();

		FJointBindingHelper JointBindings{ SkelIdentifier, JointNames, static_cast<float>(Options.TimeCode.GetValue()) };

		// For each entry add an instance to the description
		int32 NumAdded = 0;
		for (const TSharedPtr<FMeshEntry>& Entry : NativeInstancedEntries)
		{
			int32* PartIndex = MeshUidToPartIndexTable.Find(Entry->NodeUid);
			if (!PartIndex)
			{
				continue;
			}

			// If we have a skeleton get joint bindings
			if (!JointNames.IsEmpty())
			{
				UE::FUsdPrim Prim = UsdStage.GetPrimAtPath(Entry->PrimPath);
				UE::FUsdPrim SkelBindingPrim = UsdStage.GetPrimAtPath(Entry->SkelBindingPath);
				if (!(Prim && SkelBindingPrim))
				{
					continue;
				}

				TArray<int32> JointIndices;
				TArray<float> JointWeights;
				int32 ElementSize;
				if (!JointBindings.Get(SkelBindingPrim, JointIndices, JointWeights, ElementSize))
				{
					continue;
				}

				TArray<UE::Interchange::FNaniteAssemblyDescription::FBoneInfluence> BoneInfluences
					= CreateBoneInfluences(JointIndices, JointWeights);

				Description.NumInfluencesPerInstance.Add(ElementSize);
				Description.BoneInfluences.Append(BoneInfluences);
			}
	
			Description.Transforms.Add(Entry->TransformStack[0] * Options.AdditionalTransform);
			Description.PartIndices.Add(*PartIndex + PartIndexStartOffset);
			NumAdded++;
		}

		if (NumAdded > 0)
		{
			Description.PartUids.Append(PartUids);
		}

		if (NumAdded != NativeInstancedEntries.Num())
		{
			const int32 NumFailed = NativeInstancedEntries.Num() - NumAdded;
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailedAddingSkelMeshInstances", "Failed to add ({0}) skeletal mesh instances for Nanite assembly '{1}'"),
				NumFailed,
				FText::FromString(TraversalResult.AssemblyRootPrimPath.GetString())
			));
		}
	}

	void GetNaniteAssemblyPayloadDataForPointInstancer(
		const UE::FUsdPrim& PointInstancerPrim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult& TraversalResult,
		const FString& SkelIdentifier,
		const TArray<FString>& JointNames,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription)
	{
		using namespace UsdToUnreal::NaniteAssemblyUtils;

		const bool bIsNewDescription = !OutNaniteAssemblyDescription.IsSet();
		const int32 PartIndexStartOffset = bIsNewDescription ? 0 : OutNaniteAssemblyDescription.GetValue().PartUids.Num();

		FNaniteAssemblyPointInstancerData PointInstancerData =
			GetPointInstancerData(
				PointInstancerPrim,
				Options,
				TraversalResult,
				SkelIdentifier,
				JointNames,
				PartIndexStartOffset
			);

		if (!PointInstancerData.bIsValid)
		{
			return;
		}

		TArray<UE::Interchange::FNaniteAssemblyDescription::FBoneInfluence> BoneInfluences 
			= CreateBoneInfluences(PointInstancerData.JointIndices, PointInstancerData.JointWeights);

		// Now we have everything needed to create a new, or append to an existing, output description...

		if (bIsNewDescription)
		{
			OutNaniteAssemblyDescription = UE::Interchange::FNaniteAssemblyDescription{};
		}
		UE::Interchange::FNaniteAssemblyDescription& Description = OutNaniteAssemblyDescription.GetValue();

		// Helper to move or append to the description arrays
		auto UpdateDescriptionArray = [bIsNewDescription]<typename T>(T& Dst, T& Src)
		{
			if (bIsNewDescription) { Dst = MoveTemp(Src); }
			else { Dst.Append(Src); }
		};

		// If all the transform stacks leading to the part meshes were i) at identity and ii) not nested, we can 
		// use the arrays of part uids as-is.
		if (!PointInstancerData.ProtoRemappingInfo.IsRemapped())
		{
			UpdateDescriptionArray(Description.Transforms, PointInstancerData.Transforms);
			UpdateDescriptionArray(Description.PartIndices, PointInstancerData.ProtoIndices);
			UpdateDescriptionArray(Description.PartUids, PointInstancerData.ProtoRemappingInfo.PartUids);
			UpdateDescriptionArray(Description.BoneInfluences, BoneInfluences);
			UpdateDescriptionArray(Description.NumInfluencesPerInstance, PointInstancerData.NumInfluencesPerInstance);
			return;
		}
		else if (TraversalResult.IsMeshType(ENaniteAssemblyMeshType::SkeletalMesh))
		{
			// Remapped prototypes is unexpected for skeletal mesh assemblies here because Interchange always
			// combines the USD meshes bound to the same skeleton into a single uasset. As a result there
			// should only be a single part/uasset slot required for each prototype.
			return;
		}
		
		// Otherwise generate expanded arrays incorporating the appropriate transform stack for each part.
		TArray<FTransform> RemappedTransforms;
		TArray<int32> RemappedPartIndices;
		TArray<TArray<FString>>& RemappedPartUids = PointInstancerData.ProtoRemappingInfo.PartUids;

		TArray<FTransform>& LocalTransforms = PointInstancerData.ProtoRemappingInfo.LocalTransforms;
		TMap<int32, FInt32Range>& RangeTable = PointInstancerData.ProtoRemappingInfo.OriginalPrototypeIndexToNewPartRangeTable;

		// Submit each original transform/prototype multiple times, once per part...
		for (int32 Index = 0; Index < PointInstancerData.Transforms.Num(); ++Index)
		{
			const FTransform& InstanceTransform = PointInstancerData.Transforms[Index];
			int32 OriginalProtoIndex = PointInstancerData.ProtoIndices[Index] - PartIndexStartOffset;

			if (OriginalProtoIndex >= 0 && ensure(RangeTable.Contains(OriginalProtoIndex)))
			{
				const FInt32Range& PartIndexRange = RangeTable.FindChecked(OriginalProtoIndex);
				const int32 Start = PartIndexRange.GetLowerBound().GetValue();
				const int32 End = PartIndexRange.GetUpperBound().GetValue();
				for (int32 CurrentIndex = Start; CurrentIndex < End; ++CurrentIndex)
				{
					const int32 OriginalCurrentIndex = CurrentIndex - PartIndexStartOffset;
					if (ensure(LocalTransforms.IsValidIndex(OriginalCurrentIndex)))
					{
						RemappedTransforms.Add(LocalTransforms[OriginalCurrentIndex] * InstanceTransform);
						RemappedPartIndices.Add(CurrentIndex);
					}
				}
			}
		}

		UpdateDescriptionArray(Description.Transforms, RemappedTransforms);
		UpdateDescriptionArray(Description.PartIndices, RemappedPartIndices);
		UpdateDescriptionArray(Description.PartUids, RemappedPartUids);
	}

	bool GetStaticMeshPayloadData(
		FString PayloadKey,
		UInterchangeUsdContext& UsdContext,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FMeshDescription& OutMeshDescription,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SchemaHandlerUtils::GetStaticMeshPayloadData)

		const bool bIsLODMesh = CheckAndChopPayloadPrefix(PayloadKey, LODPrefix);
		const bool bIsPrimitiveShape = CheckAndChopPayloadPrefix(PayloadKey, PrimitiveShapePrefix);

		const FString& PrimPath = PayloadKey;
		UE::FUsdPrim Prim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*PrimPath});
		if (!Prim)
		{
			if (bIsLODMesh)
			{
				Prim = TryGettingInactiveLODPrim(PrimPath, UsdContext);
			}

			if (!Prim)
			{
				return false;
			}
		}

		FMeshDescription TempMeshDescription;
		FStaticMeshAttributes StaticMeshAttributes(TempMeshDescription);
		StaticMeshAttributes.Register();

		// Get Nanite assembly static mesh data, if available

		const FString MeshNodeUid = UsdContext.MakeAssetNodeUid(Prim, MeshPrefix);
		FTraversalInfo* MeshInfo;
		{
			FReadScopeLock ReadLock{ UsdContext.NodeUidToCachedTraversalInfoLock };
			MeshInfo = UsdContext.NodeUidToCachedTraversalInfo.Find(MeshNodeUid);
		}

		if (MeshInfo && MeshInfo->NaniteAssemblyTraversalResult)
		{
			if (const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult* TraversalResult = MeshInfo->NaniteAssemblyTraversalResult.Get())
			{
				static const FString UnusedSkelIdentifier;
				static const TArray<FString> UnusedJointNames;

				// Add native instance data to the output description
				GetNaniteAssemblyPayloadDataForPrims(
					UsdContext.GetUsdStage(),
					Options,
					*TraversalResult,
					UnusedSkelIdentifier,
					UnusedJointNames,
					OutNaniteAssemblyDescription);

				// Add pointinstancer data to the output description
				for (const UE::FSdfPath& TopLevelPointInstancerPath : TraversalResult->GetTopLevelPointInstancerPaths())
				{
					UE::FUsdPrim PointInstancerPrim = UsdContext.GetUsdStage().GetPrimAtPath(TopLevelPointInstancerPath);
					GetNaniteAssemblyPayloadDataForPointInstancer(
						PointInstancerPrim,
						Options,
						*TraversalResult,
						UnusedSkelIdentifier,
						UnusedJointNames,
						OutNaniteAssemblyDescription);
				}

				// Right now we only support empty static mesh base meshes, since the NaniteAssemblyRootAPI schema only 
				// allows the user to specify a base mesh for the skeletal mesh case.
				OutMeshDescription = MoveTemp(TempMeshDescription);
				return true;
			}
		}

		UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;

		bool bSuccess = false;
		if (bIsPrimitiveShape)
		{
			bSuccess = UsdToUnreal::ConvertGeomPrimitive(Prim, TempMeshDescription, TempMaterialInfo, Options);
		}
		else
		{
			bSuccess = UsdToUnreal::ConvertGeomMesh(Prim, TempMeshDescription, TempMaterialInfo, Options);
		}
		if (!bSuccess)
		{
			return false;
		}

		UsdUtils::FUsdPrimMaterialAssignmentInfo* AssignmentPtr = &TempMaterialInfo;
		if (bIsLODMesh)
		{
			// Use our cached material assignments instead of whatever we pull from ConvertGeomMesh
			// because if we're in an LOD mesh then we may be reading from a temp stage, that has a
			// population mask that may not include the material, meaning ConvertGeomMesh may have failed
			// to resolve all the bindings. The cached assignments come from AddMeshNode step, where
			// we switch the active variant on the current stage and so get nice material bindings that
			// resolve normally.
			// TODO: We may not need this anymore, as we don't use population masks to open the LOD stages anymore
			//
			// Note that we can't even use the info cache here, because it wouldn't have cached info
			// about the inactive LOD variants
			if (UsdUtils::FUsdPrimMaterialAssignmentInfo* CachedInfo = UsdContext.CachedMaterialAssignments.Find(PrimPath))
			{
				AssignmentPtr = CachedInfo;
			}
		}

		OutMeshDescription = MoveTemp(TempMeshDescription);

		FixMaterialSlotNames(OutMeshDescription, AssignmentPtr->Slots, UsdContext);

		return true;
	}

	FTransform GetCombinedTransform(const UE::FUsdPrim& Ancestor, const UE::FUsdPrim& Descendant)
	{
		FTransform Transform = FTransform::Identity;

		if (!Ancestor || !Descendant)
		{
			return Transform;
		}

		bool bResetTransformStack = false;
		UsdToUnreal::ConvertXformable(Descendant.GetStage(), UE::FUsdTyped(Descendant), Transform, UsdUtils::GetEarliestTimeCode(), &bResetTransformStack);

		if (Ancestor == Descendant)
		{
			return Transform;
		}
		else
		{
			return Transform * GetCombinedTransform(Ancestor, Descendant.GetParent());
		}
	}

	void SetPrimPath(UInterchangeBaseNode& BaseNode, const FString& PrimPath)
	{
		// CreateUserDefinedAttribute is strictly create-if-not-exists. Since we're cloning nodes
		// that already have prim path attributes, we have to remove the old value first if we want
		// to add new ones
		UInterchangeUserDefinedAttributesAPI::RemoveUserDefinedAttribute(&BaseNode, PrimPathAttributeKeyString);

		TOptional<FString> PayloadKey;
		UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(&BaseNode, PrimPathAttributeKeyString, PrimPath, PayloadKey);
	}

	FString GetPrimPath(const UInterchangeBaseNode& BaseNode)
	{
		FString PrimPath;
		TOptional<FString> PayloadKey;
		UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(&BaseNode, PrimPathAttributeKeyString, PrimPath, PayloadKey);
		return PrimPath;
	}

	FString GetMeshPrimPath(const UInterchangeBaseNode& BaseNode)
	{
		FString MeshPrimPath = GetPrimPath(BaseNode);
		if (MeshPrimPath.IsEmpty())
		{
			MeshPrimPath = GetMeshPrimPathFromNodeUid(BaseNode.GetUniqueID());
		}
		return MeshPrimPath;
	}

}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
