// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/SkeletonSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDObjectUtils.h"
#include "USDSkeletalDataConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelBinding.h"
#include "UsdWrappers/UsdSkelBlendShape.h"
#include "UsdWrappers/UsdSkelBlendShapeQuery.h"
#include "UsdWrappers/UsdSkelCache.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeCommonAnimationPayload.h"

#include "Async/ParallelFor.h"
#include "InterchangeJointNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "SkeletonSchemaHandler"

namespace UE::SkelSkeletonSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	const UInterchangeSkeletalAnimationTrackNode* AddSkeletalAnimationNode(
		const UE::FUsdSkelSkeletonQuery& SkeletonQuery,
		const TMap<FString, TPair<FString, int32>>& UsdBoneToUidAndBoneIndex,
		UInterchangeJointNode& RootJointSceneNode,
		FTraversalInfo& Info,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return nullptr;
		}

		UE::FUsdSkelAnimQuery AnimQuery = SkeletonQuery.GetAnimQuery();
		if (!AnimQuery)
		{
			return nullptr;
		}

		UE::FUsdPrim SkelAnimationPrim = AnimQuery.GetPrim();
		if (!SkelAnimationPrim)
		{
			return nullptr;
		}

		UE::FUsdPrim SkeletonPrim = SkeletonQuery.GetSkeleton();
		if (!SkeletonPrim)
		{
			return nullptr;
		}

		UE::FUsdStage Stage = SkeletonPrim.GetStage();

		const FString SkelAnimationName = SkelAnimationPrim.GetName().ToString();
		const FString SkeletonPrimPath = SkeletonPrim.GetPrimPath().GetString(); // TODO: This won't work with skeletal instancing
		const FString NodeUidSuffix = FString::Printf(TEXT("\\%s"), *SkeletonPrimPath);
		const FString SkelAnimNodeUid = UsdContext.MakeAssetNodeUid(SkelAnimationPrim, AnimationTrackPrefix, NodeUidSuffix);
		UInterchangeSkeletalAnimationTrackNode* SkelAnimNode = GetExistingNode<UInterchangeSkeletalAnimationTrackNode>(*NodeContainer, SkelAnimNodeUid);
		if (!SkelAnimNode)
		{
			SkelAnimNode = NewObject<UInterchangeSkeletalAnimationTrackNode>(NodeContainer);
			NodeContainer->SetupNode(SkelAnimNode, SkelAnimNodeUid, SkelAnimationName, EInterchangeNodeContainerType::TranslatedAsset);
		}
		UE::Interchange::USD::SetPrimPath(*SkelAnimNode, SkelAnimationPrim.GetPrimPath().GetString());

		SkelAnimNode->SetCustomSkeletonNodeUid(RootJointSceneNode.GetUniqueID());

		AccumulatedInfo.PrimAssetNodes.Add(SkelAnimNode);

		// Time info
		{
			double TimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
			if (TimeCodesPerSecond == 0.0)
			{
				TimeCodesPerSecond = 24.0;
			}

			SkelAnimNode->SetCustomAnimationSampleRate(TimeCodesPerSecond);

			TOptional<double> StartTimeCode;
			TOptional<double> StopTimeCode;

			// For now we don't generate LevelSequences for sublayers and will instead put everything on a single
			// LevelSequence for the entire stage, so we don't need to care so much about sublayer offset/scale like
			// UsdToUnreal::ConvertSkelAnim does
			TArray<double> JointTimeSamples;
			if (AnimQuery.GetJointTransformTimeSamples(JointTimeSamples) && JointTimeSamples.Num() > 0)
			{
				StartTimeCode = JointTimeSamples[0];
				StopTimeCode = JointTimeSamples[JointTimeSamples.Num() - 1];
			}
			TArray<double> BlendShapeTimeSamples;
			if (AnimQuery.GetBlendShapeWeightTimeSamples(BlendShapeTimeSamples) && BlendShapeTimeSamples.Num() > 0)
			{
				StartTimeCode = FMath::Min(BlendShapeTimeSamples[0], StartTimeCode.Get(TNumericLimits<double>::Max()));
				StopTimeCode = FMath::Max(BlendShapeTimeSamples[BlendShapeTimeSamples.Num() - 1], StopTimeCode.Get(TNumericLimits<double>::Lowest()));
			}

			if (StartTimeCode.IsSet())
			{
				SkelAnimNode->SetCustomAnimationStartTime(StartTimeCode.GetValue() / TimeCodesPerSecond);
			}
			if (StopTimeCode.IsSet())
			{
				SkelAnimNode->SetCustomAnimationStopTime(StopTimeCode.GetValue() / TimeCodesPerSecond);
			}
		}

		// Joint animation
		TArray<FString> UsdJointOrder = AnimQuery.GetJointOrder();
		for (const FString& FullAnimatedBoneName : UsdJointOrder)
		{
			const TPair<FString, int32>* FoundPair = UsdBoneToUidAndBoneIndex.Find(FullAnimatedBoneName);
			if (!FoundPair)
			{
				continue;
			}

			const FString& BoneSceneNodeUid = FoundPair->Key;
			int32 UsdSkeletonOrderBoneIndex = FoundPair->Value;

			const FString BoneAnimPayloadKey = SkeletonPrimPath + TEXT("\\") + LexToString(UsdSkeletonOrderBoneIndex);

			// When retrieving the payload later, We'll need that bone's index within the Skeleton prim to index into the
			// InUsdSkeletonQuery.ComputeJointLocalTransforms() results.
			// Note that we're describing joint transforms with baked frames here. It would have been possible to use transform
			// curves, but that may have lead to issues when interpolating problematic joint transforms. Instead, we'll bake
			// using USD, and let it interpolate the transforms however it wants
			SkelAnimNode->SetAnimationPayloadKeyForSceneNodeUid(BoneSceneNodeUid, BoneAnimPayloadKey, EInterchangeAnimationPayLoadType::BAKED);
		}

		// Morph targets
		{
			UE::FUsdSkelBinding SkelBinding;
			const bool bTraverseInstanceProxies = true;
			bool bSuccess = Info.FurthestSkelCache->ComputeSkelBinding(	   //
				Info.ResolveClosestParentSkelRoot(Stage),
				SkeletonPrim,
				SkelBinding,
				bTraverseInstanceProxies
			);
			if (!bSuccess)
			{
				return SkelAnimNode;
			}

			TArray<FString> SkelAnimChannelOrder = AnimQuery.GetBlendShapeOrder();

			TMap<FString, int32> SkelAnimChannelIndices;
			SkelAnimChannelIndices.Reserve(SkelAnimChannelOrder.Num());
			for (int32 ChannelIndex = 0; ChannelIndex < SkelAnimChannelOrder.Num(); ++ChannelIndex)
			{
				const FString& ChannelName = SkelAnimChannelOrder[ChannelIndex];
				SkelAnimChannelIndices.Add(ChannelName, ChannelIndex);
			}

			TArray<UE::FUsdSkelSkinningQuery> SkinningTargets = SkelBinding.GetSkinningTargets();
			for (const UE::FUsdSkelSkinningQuery& SkinningTarget : SkinningTargets)
			{
				// USD lets you "skin" anything that can take the SkelBindingAPI, but we only care about Mesh here as
				// those are the only ones that can have blendshapes
				UE::FUsdPrim MeshPrim = SkinningTarget.GetPrim();
				if (!MeshPrim.IsA(TEXT("Mesh")))
				{
					continue;
				}

				TArray<FString> BlendShapeChannels;
				bool bInnerSucces = SkinningTarget.GetBlendShapeOrder(BlendShapeChannels);
				if (!bInnerSucces)
				{
					continue;
				}

				TArray<UE::FSdfPath> Targets;
				{
					UE::FUsdRelationship BlendShapeTargetsRel = SkinningTarget.GetBlendShapeTargetsRel();
					if (!BlendShapeTargetsRel)
					{
						continue;
					}
					bInnerSucces = BlendShapeTargetsRel.GetTargets(Targets);
					if (!bInnerSucces)
					{
						continue;
					}
				}

				if (BlendShapeChannels.Num() != Targets.Num())
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"SkippingMorphTargetCurves",
							"Skipping morph target curves for animation of skinned mesh '{0}' because the number of entries in the 'skel:blendShapes' attribute ({1}) doesn't match the number of entries in the 'skel:blendShapeTargets' attribute ({2})"
						),
						FText::FromString(MeshPrim.GetPrimPath().GetString()),
						BlendShapeChannels.Num(),
						Targets.Num()
					));

					continue;
				}

				for (int32 BlendShapeIndex = 0; BlendShapeIndex < Targets.Num(); ++BlendShapeIndex)
				{
					const FString& ChannelName = BlendShapeChannels[BlendShapeIndex];
					int32* FoundSkelAnimChannelIndex = SkelAnimChannelIndices.Find(ChannelName);
					if (!FoundSkelAnimChannelIndex)
					{
						// This channel is not animated by this SkelAnimation prim
						continue;
					}

					// Note that we put no inbetween name on the MorphTargetUid: We only need to emit the morph target curve payloads
					// for the main shapes: We'll provide the inbetween "positions" when providing the curve and Interchange computes
					// the inbetween curves automatically
					const FString BlendShapePath = Targets[BlendShapeIndex].GetString();
					const FString MorphTargetUid = GetMorphTargetMeshNodeUid(UsdContext, MeshPrim, BlendShapeIndex);
					const FString PayloadKey = GetMorphTargetCurvePayloadKey(SkeletonPrimPath, *FoundSkelAnimChannelIndex, BlendShapePath);

					SkelAnimNode->SetAnimationPayloadKeyForMorphTargetNodeUid(	  //
						MorphTargetUid,
						PayloadKey,
						EInterchangeAnimationPayLoadType::MORPHTARGETCURVE
					);
				}
			}
		}

		return SkelAnimNode;
	}

	void ConvertSkeletonPrim(
		const UE::FUsdPrim& Prim,
		UInterchangeSceneNode& SkeletonPrimNode,
		FTraversalInfo& Info,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertSkeletonPrim)

		// If we're not inside of a SkelRoot, the skeleton shouldn't really do anything
		if (!Info.ClosestParentSkelRootPath.IsValid())
		{
			return;
		}

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return;
		}

		// By the time we get here we've already emitted a scene node for the skeleton prim itself, so we just
		// need to emit a node hierarchy that mirrors the joints.

		const FString SkeletonPrimPath = Prim.GetPrimPath().GetString();

#if WITH_EDITOR
		// Convert the skeleton bones/joints into ConvertedData
		UE::FUsdSkelSkeletonQuery SkelQuery = Info.FurthestSkelCache->GetSkelQuery(Prim);
		const bool bEnsureAtLeastOneBone = false;
		const bool bEnsureSingleRootBone = false;
		UsdToUnreal::FUsdSkeletonData ConvertedData;
		const bool bSuccess = UsdToUnreal::ConvertSkeleton(SkelQuery, ConvertedData, bEnsureAtLeastOneBone, bEnsureSingleRootBone);
		if (!bSuccess || ConvertedData.Bones.Num() == 0)
		{
			return;
		}

		// If we have multiple root bones, we need to add a single "true" root bone here, and then just add all the
		// input root bones as children. We need to make sure our root bone has a unique name, but once that is done things
		// should work out fine (including offsetting the vertex bone influence indices) because Interchange will compare these
		// bone names with the bone names we return from GetSkeletalMeshPayloadData() and remap the indices by name
		bool bHadMultipleRootBones = false;
		{
			TArray<int32> RootBoneIndices;
			for (int32 BoneIndex = 0; BoneIndex < ConvertedData.Bones.Num(); ++BoneIndex)
			{
				const UsdToUnreal::FUsdSkeletonData::FBone& Bone = ConvertedData.Bones[BoneIndex];
				if (Bone.ParentIndex == INDEX_NONE)
				{
					RootBoneIndices.Add(BoneIndex); 
				}
			}

			if (RootBoneIndices.Num() > 1)
			{
				bHadMultipleRootBones = true;

				UsdToUnreal::FUsdSkeletonData::FBone TrueRoot;
				TrueRoot.Name = UE::Interchange::USD::RootBoneUidSuffix;

				TrueRoot.ChildIndices = RootBoneIndices;
				ConvertedData.Bones.Insert(TrueRoot, 0);

				// This will also patch up our TrueRoot bone, which is great as it fixes
				// up its ChildIndices too
				for (UsdToUnreal::FUsdSkeletonData::FBone& Bone : ConvertedData.Bones)
				{
					Bone.ParentIndex += 1;
					for (int32& ChildIndex : Bone.ChildIndices)
					{
						ChildIndex += 1;
					}
				}

				// We need to fixup the true root's parentindex here though
				ConvertedData.Bones[0].ParentIndex = INDEX_NONE;
			}
		}

		// Maps from the USD-style full bone name (e.g. "shoulder/elbow/hand") to the Uid we used for
		// the corresponding scene node, and the bone's index on the skeleton's joint order.
		// We'll need this to parse skeletal animations, if any
		TMap<FString, TPair<FString, int32>> UsdBoneToUidAndUsdBoneIndex;
		UInterchangeJointNode* RootBoneSceneNode = nullptr;

		// Recursively traverse ConvertedData spawning the joint translated nodes
		TFunction<void(int32, UInterchangeSceneNode&, const FString&)> RecursiveTraverseBones = nullptr;
		RecursiveTraverseBones = [&RecursiveTraverseBones,
								  &UsdBoneToUidAndUsdBoneIndex,
								  &SkeletonPrimPath,
								  &ConvertedData,
								  &NodeContainer,
								  &RootBoneSceneNode,
								  bHadMultipleRootBones,
								  &AccumulatedInfo
		](int32 BoneIndex, UInterchangeSceneNode& ParentNode, const FString& UsdBonePath)
		{
			const UsdToUnreal::FUsdSkeletonData::FBone& Bone = ConvertedData.Bones[BoneIndex];

			const bool bIsRootBone = BoneIndex == 0;

			// Name of the individual bone, as-is from USD. Our "true root" becomes just "" here, as it doesn't exist in USD
			const FString UsdBoneName = (bIsRootBone && bHadMultipleRootBones) ? TEXT("") : Bone.Name;

			// Concatenate a full "bone path" here for uniqueness
			// This now matches USD bone paths (e.g. "shoulder/elbow/hand")
			const FString ConcatUsdBonePath = UsdBonePath.IsEmpty() ? UsdBoneName : UsdBonePath + TEXT("/") + UsdBoneName;

			// Append our "true root" bone if we have one (e.g. shoulder/elbow/hand -> Root/shoulder/elbow/hand)
			const FString FullConcatUsdBonePath = (bHadMultipleRootBones)
				? RootBoneUidSuffix + (bIsRootBone ? TEXT("") : TEXT("/") + ConcatUsdBonePath)
				: ConcatUsdBonePath;

			// The UID for a root bone (whether "true root" or not) is always \Bone\<SkeletonPrimPath>/RootBoneUidSuffix (e.g. "\Bone\<SkeletonPrimPath>/Root")
			// given our call to MakeRootBoneNodeUid().
			// For any other bone, it becomes \Bone\<SkeletonPrimPath>/FullConcat (e.g. "\Bone\<SkeletonPrimPath>/<UsdRootBoneName>/Elbow/Hand")
			//
			// Note that this may look slightly odd for non-multiple-root-bone cases, because the root bone UID will be e.g. "\Bone\<SkeletonPrimPath>/Root", and its
			// immediate child's will be "\Bone\<SkeletonPrimPath>/<UsdRootBoneName>/Elbow", so *not* "\Bone\<SkeletonPrimPath>/Root/Elbow".
			// This simply because it's the easiest to manipulate/construct here, but either would have worked: These are just UIDs after all.
			// The display labels are still just sensible bone names
			const FString BoneNodeUid = bIsRootBone
				? MakeRootBoneNodeUid(SkeletonPrimPath)
				: MakeBoneNodeUid(SkeletonPrimPath, FullConcatUsdBonePath);

			UInterchangeJointNode* BoneNode = GetExistingNode<UInterchangeJointNode>(*NodeContainer, BoneNodeUid);
			if (!BoneNode)
			{
				BoneNode = NewObject<UInterchangeJointNode>(NodeContainer);
				NodeContainer->SetupNode(BoneNode, BoneNodeUid, Bone.Name, EInterchangeNodeContainerType::TranslatedScene, ParentNode.GetUniqueID());
			}

			AccumulatedInfo.PrimSceneNodes.Add(BoneNode);

			// Note that we use our rest transforms for the Interchange bind pose as well: This because Interchange
			// will put this on the RefSkeleton and so it will make its way to the Skeleton asset. We already kind
			// of bake in our skeleton bind pose directly into our skinned mesh, so we really just want to put the
			// rest pose on the skeleton asset/ReferenceSkeleton
			BoneNode->SetBindPoseLocalTransform(NodeContainer, Bone.LocalBindTransform);
			BoneNode->SetTimeZeroLocalTransform(NodeContainer, Bone.LocalBindTransform);
			BoneNode->SetCustomLocalTransform(NodeContainer, Bone.LocalBindTransform);

			// These will be used to index into the baked transforms from USD, so we have to pretend our "true root" bone doesn't exist,
			// if we added one
			const int32 UsdBoneIndex = bHadMultipleRootBones ? BoneIndex - 1 : BoneIndex;

			UsdBoneToUidAndUsdBoneIndex.Add(ConcatUsdBonePath, {BoneNodeUid, UsdBoneIndex});
			if (bIsRootBone)
			{
				RootBoneSceneNode = BoneNode;
			}

			for (int32 ChildIndex : Bone.ChildIndices)
			{
				RecursiveTraverseBones(ChildIndex, *BoneNode, ConcatUsdBonePath);
			}
		};

		// Start traversing from the root bone (at this point we know we have exactly one)
		{
			const FString BonePathRoot = TEXT("");
			const int32 BoneIndex = 0;
			RecursiveTraverseBones(BoneIndex, SkeletonPrimNode, BonePathRoot);
		}
		if (!RootBoneSceneNode)
		{
			return;
		}

		// Interchange will abort parsing skeletons that don't have unique names for each bone. If the user has that
		// on their actual skeleton, then that's just invalid data and we can just let it fail and emit the error message.
		// However, we don't want to end up with duplicate bone names and fail to parse when the duplicate bone is due to
		// our extra true root bone... In this case, here we just change the display label of that node itself to be unique
		// (which is used for the bone name)
		if (bHadMultipleRootBones && RootBoneSceneNode)
		{
			TSet<FString> UsedBoneNames;
			for (int32 BoneIndex = 1; BoneIndex < ConvertedData.Bones.Num(); ++BoneIndex)
			{
				const UsdToUnreal::FUsdSkeletonData::FBone& Bone = ConvertedData.Bones[BoneIndex];
				UsedBoneNames.Add(Bone.Name);
			}

			FString NewSkeletonPrimName = UsdUnreal::ObjectUtils::GetUniqueName(UE::Interchange::USD::RootBoneUidSuffix, UsedBoneNames);
			if (NewSkeletonPrimName != UE::Interchange::USD::RootBoneUidSuffix)
			{
				const FString RootBoneUid = MakeRootBoneNodeUid(SkeletonPrimPath);
				RootBoneSceneNode->SetDisplayLabel(NewSkeletonPrimName);
			}
		}

		// Handle SkelAnimation prims, if we have any bound for this Skeleton
		const UInterchangeSkeletalAnimationTrackNode* SkelAnimNode = AddSkeletalAnimationNode(
			SkelQuery,
			UsdBoneToUidAndUsdBoneIndex,
			*RootBoneSceneNode,
			Info,
			AccumulatedInfo,
			UsdContext
		);
		if (SkelAnimNode)
		{
			RootBoneSceneNode->SetCustomAnimationAssetUidToPlay(SkelAnimNode->GetUniqueID());
		}

		// Cache our joint names in order, as this is needed when generating skeletal mesh payloads
		Info.SkelJointUsdNames = MakeShared<TArray<FString>>();
		Info.SkelJointUsdNames->Reserve(ConvertedData.Bones.Num());
		for (const UsdToUnreal::FUsdSkeletonData::FBone& Bone : ConvertedData.Bones)
		{
			// The joint names that the skeletal mesh payload needs to retrieve should match exactly the USD bone
			// structure (and so correspond to the vertex bone influence indices). We really don't want our extra
			// "true root" bone to show up in there, or else we will mislead Interchange
			if (bHadMultipleRootBones && Bone.ParentIndex == INDEX_NONE)
			{
				continue;
			}

			Info.SkelJointUsdNames->Add(Bone.Name);
		}

		// Prefer flagging the Skeleton prim itself as the BoundSkeletonPrimPath at this point in the hierarchy, even
		// preferring it over any explicit skel:skeleton relationship.
		// This does not seem technically correct, but is useful in case the Skeleton prim has a skel:animationSource
		// relationship directly on it, which seems to animate in usdview and is advertised as a supported case.
		// References:
		// - https://github.com/usd-wg/assets/blob/main/test_assets/USDZ/CesiumMan/CesiumMan.usdz
		// - https://openusd.org/release/api/_usd_skel__o_m.html
		// - https://github.com/PixarAnimationStudios/OpenUSD/issues/3532
		if (UE::FUsdPrim SkeletonPrim = SkelQuery.GetSkeleton())
		{
			Info.BoundSkeletonPrimPath = MakeShared<FString>(SkeletonPrim.GetPrimPath().GetString());
		}

		{
			// Cache the skeleton global transform, because if we need to bake our skeletal mesh later then we'll need to use this
			// and it's awkward to have Interchange provide this to us natively, as the "skeleton prim node" here, which corresponds
			// to the "first scene node parent of the root joint transform" doesn't really mean anything for the other formats...
			const FTransform GlobalOffsetTransform;
			SkeletonPrimNode.GetCustomGlobalTransform(NodeContainer, GlobalOffsetTransform, Info.SceneGlobalTransform);

			FWriteScopeLock ScopedInfoWriteLock{UsdContext.NodeUidToCachedTraversalInfoLock};
			UsdContext.NodeUidToCachedTraversalInfo.Add(SkeletonPrimNode.GetUniqueID(), Info);
		}
#endif	  // WITH_EDITOR
	}

	bool GetJointAnimationCurvePayloadData(
		const TArray<const UE::Interchange::FAnimationPayloadQuery*>& Queries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetJointAnimationCurvePayloadData)

		if (Queries.Num() == 0)
		{
			return false;
		}

		// We expect all queries to be for the same skeleton, and have the same timing parameters,
		// since they were grouped up by HashAnimPayloadQuery, so let's just grab one for the params
		const UE::Interchange::FAnimationPayloadQuery* FirstQuery = Queries[0];

		// Parse payload key.
		// Here is takes the form "<skeleton prim path>\<joint index in skeleton order>"
		TArray<FString> PayloadKeyTokens;
		FirstQuery->PayloadKey.UniqueId.ParseIntoArray(PayloadKeyTokens, TEXT("\\"));
		if (PayloadKeyTokens.Num() != 2)
		{
			return false;
		}

		// Fetch our cached skeleton query
		const FString& SkeletonPrimPath = PayloadKeyTokens[0];
		UE::FUsdSkelSkeletonQuery SkelQuery;
		{
			FReadScopeLock ReadLock{UsdContext.NodeUidToCachedTraversalInfoLock};

			UE::FUsdPrim SkeletonPrim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath(*SkeletonPrimPath));
			const FTraversalInfo* Info = SkeletonPrim
				? UsdContext.NodeUidToCachedTraversalInfo.Find(UsdContext.MakeSceneNodeUid(SkeletonPrim))
				: nullptr;
			if (!Info)
			{
				return false;
			}

			SkelQuery = Info->ResolveSkelQuery(UsdContext.GetUsdStage());
			if (!SkelQuery)
			{
				return false;
			}
		}

		UE::FUsdPrim SkeletonPrim = SkelQuery.GetPrim();
		UE::FUsdStage Stage = SkeletonPrim.GetStage();
		FUsdStageInfo StageInfo{Stage};

		// Compute the bake ranges and intervals
		double TimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
		if (TimeCodesPerSecond == 0.0)
		{
			TimeCodesPerSecond = 24.0;
		}
		double BakeFrequency = FirstQuery->TimeDescription.BakeFrequency;
		double RangeStartSeconds = FirstQuery->TimeDescription.RangeStartSecond;
		double RangeStopSeconds = FirstQuery->TimeDescription.RangeStopSecond;
		double SectionLengthSeconds = RangeStopSeconds - RangeStartSeconds;
		double StartTimeCode = RangeStartSeconds * TimeCodesPerSecond;
		const int32 NumBakedFrames = FMath::RoundToInt(FMath::Max(SectionLengthSeconds * TimeCodesPerSecond + 1.0, 1.0));
		double TimeCodeIncrement = (1.0 / BakeFrequency) * TimeCodesPerSecond;

		// Bake all joint transforms via USD into arrays for each separate joint (in whatever order SkelQuery gives us)
		TArray<TArray<FTransform>> BakedTransforms;
		for (int32 FrameIndex = 0; FrameIndex < NumBakedFrames; ++FrameIndex)
		{
			const double FrameTimeCode = StartTimeCode + FrameIndex * TimeCodeIncrement;

			TArray<FTransform> TransformsForTimeCode;
			bool bSuccess = SkelQuery.ComputeJointLocalTransforms(TransformsForTimeCode, FrameTimeCode);
			if (!bSuccess)
			{
				break;
			}

			for (FTransform& Transform : TransformsForTimeCode)
			{
				Transform = UsdUtils::ConvertTransformToUESpace(StageInfo, Transform);
			}

			// Setup our BakedTransforms in here, because we may actually get more or less transforms
			// from the SkeletonQuery than our AnimSequence wants/expects, given that it can specify
			// its own animated joint order
			int32 NumSkelJoints = TransformsForTimeCode.Num();
			if (FrameIndex == 0)
			{
				BakedTransforms.SetNum(NumSkelJoints);
				for (int32 JointIndex = 0; JointIndex < NumSkelJoints; ++JointIndex)
				{
					BakedTransforms[JointIndex].SetNum(NumBakedFrames);
				}
			}

			// Transpose our baked transforms into the arrays we'll eventually return
			for (int32 JointIndex = 0; JointIndex < NumSkelJoints; ++JointIndex)
			{
				BakedTransforms[JointIndex][FrameIndex] = TransformsForTimeCode[JointIndex];
			}
		}

		// Finally build our payload data return values by picking the desired baked arrays with the payload joint indices
		OutPayloadData.Reset(Queries.Num());
		for (int32 QueryIndex = 0; QueryIndex < Queries.Num(); ++QueryIndex)
		{
			const UE::Interchange::FAnimationPayloadQuery* Query = Queries[QueryIndex];

			FString IndexStr = Query->PayloadKey.UniqueId.RightChop(SkeletonPrimPath.Len() + 1);	// Also skip the '\'
			int32 JointIndex = INDEX_NONE;
			bool bLexed = LexTryParseString(JointIndex, *IndexStr);
			if (!bLexed)
			{
				continue;
			}

			UE::Interchange::FAnimationPayloadData& PayloadData = OutPayloadData.Emplace_GetRef(Query->SceneNodeUniqueID, Query->PayloadKey);
			PayloadData.BakeFrequency = BakeFrequency;
			PayloadData.RangeStartTime = RangeStartSeconds;
			PayloadData.RangeEndTime = RangeStopSeconds;

			if (BakedTransforms.IsValidIndex(JointIndex))
			{
				PayloadData.Transforms = MoveTemp(BakedTransforms[JointIndex]);
			}
		}

		return true;
	}

	bool GetMorphTargetAnimationCurvePayloadData(
		const FString& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetMorphTargetAnimationCurvePayloadData)

		// Here we must output the morph target curve for a particular channel and skinning target, i.e.
		// the connection of a SkelAnimation blend shape channel to a particular Mesh prim.

		// These payload keys were generated from GetMorphTargetCurvePayloadKey(), so they take the form
		// "<skeleton prim path>\<skel anim channel index>\<blend shape path>"
		TArray<FString> PayloadKeyTokens;
		PayloadKey.ParseIntoArray(PayloadKeyTokens, TEXT("\\"));
		if (PayloadKeyTokens.Num() != 3)
		{
			return false;
		}
		const FString& SkeletonPrimPath = PayloadKeyTokens[0];
		const FString& AnimChannelIndexStr = PayloadKeyTokens[1];
		const FString& BlendShapePath = PayloadKeyTokens[2];

		const UE::FUsdStage UsdStage = UsdContext.GetUsdStage();

		int32 SkelAnimChannelIndex = INDEX_NONE;
		bool bLexed = LexTryParseString(SkelAnimChannelIndex, *AnimChannelIndexStr);

		UE::FUsdPrim BlendShapePrim = UsdStage.GetPrimAtPath(UE::FSdfPath{*BlendShapePath});
		UE::FUsdSkelBlendShape BlendShape{BlendShapePrim};
		if (!BlendShape || !bLexed || SkelAnimChannelIndex == INDEX_NONE)
		{
			return false;
		}
		const FString BlendShapeName = BlendShapePrim.GetName().ToString();

		// Fill in the actual morph target curve
		UE::FUsdSkelAnimQuery AnimQuery;
		{
			UE::FUsdSkelSkeletonQuery SkelQuery;
			{
				FReadScopeLock ReadLock{UsdContext.NodeUidToCachedTraversalInfoLock};

				UE::FUsdPrim SkeletonPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*SkeletonPrimPath));
				const FTraversalInfo* Info = SkeletonPrim
					? UsdContext.NodeUidToCachedTraversalInfo.Find(UsdContext.MakeSceneNodeUid(SkeletonPrim))
					: nullptr;
				if (!Info)
				{
					return false;
				}

				SkelQuery = Info->ResolveSkelQuery(UsdStage);
				if (!SkelQuery)
				{
					return false;
				}
			}

			AnimQuery = SkelQuery.GetAnimQuery();
			if (!AnimQuery)
			{
				return false;
			}

			TArray<double> TimeCodes;
			bool bSuccess = AnimQuery.GetBlendShapeWeightTimeSamples(TimeCodes);
			if (!bSuccess)
			{
				return false;
			}

			OutPayloadData.Curves.SetNum(1);
			FRichCurve& Curve = OutPayloadData.Curves[0];
			Curve.ReserveKeys(TimeCodes.Num());

			double TimeCodesPerSecond = UsdStage.GetTimeCodesPerSecond();
			if (TimeCodesPerSecond == 0.0)
			{
				TimeCodesPerSecond = 24.0;
			}

			const FFrameRate StageFrameRate{static_cast<uint32>(TimeCodesPerSecond), 1};
			const ERichCurveInterpMode InterpMode = (UsdStage.GetInterpolationType() == EUsdInterpolationType::Linear)
														? ERichCurveInterpMode::RCIM_Linear
														: ERichCurveInterpMode::RCIM_Constant;

			TArray<float> Weights;
			for (double TimeCode : TimeCodes)
			{
				bSuccess = AnimQuery.ComputeBlendShapeWeights(Weights, TimeCode);
				if (!bSuccess || !Weights.IsValidIndex(SkelAnimChannelIndex))
				{
					break;
				}

				int32 FrameNumber = FMath::FloorToInt(TimeCode);
				float SubFrameNumber = TimeCode - FrameNumber;

				FFrameTime FrameTime{FrameNumber, SubFrameNumber};
				double FrameTimeSeconds = static_cast<float>(StageFrameRate.AsSeconds(FrameTime));

				FKeyHandle Handle = Curve.AddKey(FrameTimeSeconds, Weights[SkelAnimChannelIndex]);
				Curve.SetKeyInterpMode(Handle, InterpMode);
			}
		}

		TArray<FString> SkelAnimChannels = AnimQuery.GetBlendShapeOrder();

		// Provide inbetween names/positions for this morph target payload
		TArray<UE::FUsdSkelInbetweenShape> Inbetweens = BlendShape.GetInbetweens();
		if (Inbetweens.Num() > 0)
		{
			// Let's store them into this temp struct so that we can sort them by weight first,
			// as Interchange seems to expect that given how it will pass these right along into
			// ResolveWeightsForBlendShape inside InterchangeAnimSequenceFactory.cpp
			struct FInbetweenAndPosition
			{
				FString Name;
				float Position;
			};
			TArray<FInbetweenAndPosition> ParsedInbetweens;
			ParsedInbetweens.Reset(Inbetweens.Num());

			for (const UE::FUsdSkelInbetweenShape& Inbetween : Inbetweens)
			{
				float Position = 0.5f;
				bool bSuccess = Inbetween.GetWeight(&Position);
				if (!bSuccess)
				{
					continue;
				}

				// Skip invalid positions. Note that technically positions outside the [0, 1] range seem to be allowed, but
				// they don't seem to work very well with our inbetween weights resolution function for some reason.
				// The legacy USD workflows have this exact same check though, so for consistency let's just do the same, and
				// if becomes an issue we should fix both
				if (Position > 1.0f || Position < 0.0f || FMath::IsNearlyZero(Position) || FMath::IsNearlyEqual(Position, 1.0f))
				{
					continue;
				}

				const FString MorphTargetName = BlendShapeName + TEXT("_") + Inbetween.GetAttr().GetName().ToString();
				FInbetweenAndPosition& NewEntry = ParsedInbetweens.Emplace_GetRef();
				NewEntry.Name = MorphTargetName;
				NewEntry.Position = Position;
			}

			ParsedInbetweens.Sort(
				[](const FInbetweenAndPosition& LHS, const FInbetweenAndPosition& RHS)
				{
					// It's invalid USD to author two inbetweens with the same weight, so let's ignore that case here.
					// (Reference: https://openusd.org/release/api/_usd_skel__schemas.html#UsdSkel_BlendShape)
					return LHS.Position < RHS.Position;
				}
			);

			OutPayloadData.InbetweenCurveNames.Reset(Inbetweens.Num() + 1);
			OutPayloadData.InbetweenFullWeights.Reset(Inbetweens.Num());

			// We add the main morph target curve name to InbetweenCurveNames too (having it end up one size bigger than
			// InbetweenFullWeights) as it seems like that's what Interchange expects. See CreateMorphTargetCurve within
			// InterchangeAnimSequenceFactory.cpp, and the very end of FFbxMesh::AddAllMeshes within FbxMesh.cpp
			OutPayloadData.InbetweenCurveNames.Add(BlendShapeName);

			for (const FInbetweenAndPosition& InbetweenAndPosition : ParsedInbetweens)
			{
				OutPayloadData.InbetweenCurveNames.Add(InbetweenAndPosition.Name);
				OutPayloadData.InbetweenFullWeights.Add(InbetweenAndPosition.Position);
			}
		}

		return true;
	}
}

namespace UE::Interchange::USD
{
	const FString& FSkeletonSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("SkeletonHandler");
		return HandlerName;
	}

	const FString& FSkeletonSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Skeleton");
		return SchemaName;
	}

	TOptional<bool> FSkeletonSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FSkeletonSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FSkeletonSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletonSchemaHandler::OnTranslate)

		using namespace UE::SkelSkeletonSchemaHandler::Private;

		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);

		// For now we must have a scene node in order to produce joints, which are other child scene nodes
		UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase);
		if (!SceneNode)
		{
			return false;
		}

		UE::Interchange::USD::SetPrimPath(*SceneNode, Prim.GetPrimPath().GetString());

		// Skeleton joints are separate scene nodes in Interchange, so we need to emit that node hierarchy now
		ConvertSkeletonPrim(Prim, *SceneNode, TraversalInfo, AccumulatedInfo, UsdContext);
		return true;
	}

	bool FSkeletonSchemaHandler::OnGetAnimationPayloadData(
		const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletonSchemaHandler::OnGetAnimationPayloadData)

		using namespace UE::Interchange;
		using namespace UE::SkelSkeletonSchemaHandler::Private;

		// Maps to help sorting the queries by payload type
		TArray<int32> BakeQueryIndexes;
		TArray<TArray<UE::Interchange::FAnimationPayloadData>> BakeAnimationPayloads;
		TArray<int32> CurveQueryIndexes;
		TArray<TArray<UE::Interchange::FAnimationPayloadData>> CurveAnimationPayloads;

		// Get all curves with a parallel for
		int32 PayloadCount = PayloadQueries.Num();
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadCount; ++PayloadIndex)
		{
			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];
			EInterchangeAnimationPayLoadType QueryType = PayloadQuery.PayloadKey.Type;
			if (QueryType == EInterchangeAnimationPayLoadType::BAKED)
			{
				BakeQueryIndexes.Add(PayloadIndex);
			}
			else
			{
				CurveQueryIndexes.Add(PayloadIndex);
			}
		}

		// Import the Baked curve payloads
		if (BakeQueryIndexes.Num() > 0)
		{
			int32 BakePayloadCount = BakeQueryIndexes.Num();
			TMap<FString, TArray<const UE::Interchange::FAnimationPayloadQuery*>> BatchedBakeQueries;
			BatchedBakeQueries.Reserve(BakePayloadCount);

			// Get the BAKED transform synchronously, since there is some interchange task that parallel them
			for (int32 BakePayloadIndex = 0; BakePayloadIndex < BakePayloadCount; ++BakePayloadIndex)
			{
				if (!ensure(BakeQueryIndexes.IsValidIndex(BakePayloadIndex)))
				{
					continue;
				}
				int32 PayloadIndex = BakeQueryIndexes[BakePayloadIndex];
				if (!PayloadQueries.IsValidIndex(PayloadIndex))
				{
					continue;
				}
				const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];
				check(PayloadQuery.PayloadKey.Type == EInterchangeAnimationPayLoadType::BAKED);
				// Joint transform animation queries.
				//
				// Currently we'll receive the PayloadQueries for all joints of a skeletal animation on the same GetAnimationPayloadData
				// call. Unfortunately in USD we must compute all joint transforms every time, even if all we need is data for a single
				// joint. For efficiency then, we group up all the queries for the separate joints of the same skeleton into one batch
				// task that we can resolve in one pass
				const FString BakedQueryHash = HashAnimPayloadQuery(PayloadQuery);
				TArray<const UE::Interchange::FAnimationPayloadQuery*>& Queries = BatchedBakeQueries.FindOrAdd(BakedQueryHash);
				Queries.Add(&PayloadQuery);
			}
			// Emit the batched joint transform animation tasks
			for (const TPair<FString, TArray<const UE::Interchange::FAnimationPayloadQuery*>>& BatchedBakedQueryPair : BatchedBakeQueries)
			{
				const TArray<const UE::Interchange::FAnimationPayloadQuery*>& Queries = BatchedBakedQueryPair.Value;
				TArray<UE::Interchange::FAnimationPayloadData> Result;
				GetJointAnimationCurvePayloadData(Queries, UsdContext, Result);
				BakeAnimationPayloads.Add(Result);
			}

			// Append the bake curves results
			for (TArray<UE::Interchange::FAnimationPayloadData>& AnimationPayload : BakeAnimationPayloads)
			{
				InOutPayloadData.Append(AnimationPayload);
			}
		}

		// Import normal curves
		if (CurveQueryIndexes.Num() > 0)
		{
			auto GetAnimPayloadLambda = [this, &PayloadQueries, &CurveAnimationPayloads, &UsdContext](int32 PayloadIndex)
			{
				if (!PayloadQueries.IsValidIndex(PayloadIndex))
				{
					return;
				}
				const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];
				EInterchangeAnimationPayLoadType PayloadType = PayloadQuery.PayloadKey.Type;
				if (PayloadType == EInterchangeAnimationPayLoadType::MORPHTARGETCURVE)
				{
					// Morph target curve queries.
					FAnimationPayloadData AnimationPayLoadData{PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey};
					if (GetMorphTargetAnimationCurvePayloadData(PayloadQuery.PayloadKey.UniqueId, UsdContext, AnimationPayLoadData))
					{
						CurveAnimationPayloads[PayloadIndex].Emplace(AnimationPayLoadData);
					}
				}
			};

			// Get all curves with a parallel for if there is many
			int32 CurvePayloadCount = CurveQueryIndexes.Num();
			CurveAnimationPayloads.AddDefaulted(CurvePayloadCount);
			const int32 BatchSize = 10;
			if (CurvePayloadCount > BatchSize)
			{
				const int32 NumBatches = (CurvePayloadCount / BatchSize) + 1;
				ParallelFor(
					NumBatches,
					[&CurveQueryIndexes, &GetAnimPayloadLambda](int32 BatchIndex)
					{
						int32 PayloadIndexOffset = BatchIndex * BatchSize;
						for (int32 PayloadIndex = PayloadIndexOffset; PayloadIndex < PayloadIndexOffset + BatchSize; ++PayloadIndex)
						{
							// The last batch can be incomplete
							if (!CurveQueryIndexes.IsValidIndex(PayloadIndex))
							{
								break;
							}
							GetAnimPayloadLambda(CurveQueryIndexes[PayloadIndex]);
						}
					},
					EParallelForFlags::BackgroundPriority	 // ParallelFor
				);
			}
			else
			{
				for (int32 PayloadIndex = 0; PayloadIndex < CurvePayloadCount; ++PayloadIndex)
				{
					int32 PayloadQueriesIndex = CurveQueryIndexes[PayloadIndex];
					if (PayloadQueries.IsValidIndex(PayloadQueriesIndex))
					{
						GetAnimPayloadLambda(PayloadQueriesIndex);
					}
				}
			}

			// Append the curves results
			for (TArray<UE::Interchange::FAnimationPayloadData>& AnimationPayload : CurveAnimationPayloads)
			{
				InOutPayloadData.Append(AnimationPayload);
			}
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
