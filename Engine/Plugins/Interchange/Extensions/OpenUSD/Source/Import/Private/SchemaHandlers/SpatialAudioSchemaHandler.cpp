// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/SpatialAudioSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTraversalInfo.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#include "Audio/InterchangeAudioPayloadData.h"
#include "InterchangeAudioSoundWaveNode.h"
#include "InterchangeResult.h"
#include "InterchangeResultsContainer.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usdMedia/tokens.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "SpatialAudioSchemaHandler"

static bool GInterchangeEnableUSDAudioImport = true;
static FAutoConsoleVariableRef CVarInterchangeEnableUSDAudioImport(
	TEXT("Interchange.FeatureFlags.Import.USD.Audio"),
	GInterchangeEnableUSDAudioImport,
	TEXT("Whether support for USD Media Spatial Audio is enabled.")
);

namespace UE::Interchange::USD
{
	const FString& FSpatialAudioSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("SpatialAudioHandler");
		return HandlerName;
	}

	const FString& FSpatialAudioSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("SpatialAudio");
		return SchemaName;
	}

	TOptional<bool> FSpatialAudioSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FSpatialAudioSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FSpatialAudioSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSpatialAudioSchemaHandler::OnTranslate)

		const FString NodeUid = UsdContext.MakeAssetNodeUid(Prim, SpatialAudioPrefix);
		const FString NodeName(Prim.GetName().ToString());

		if (!GInterchangeEnableUSDAudioImport)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("UnsupportedSchemaType", "Prim '{0}' has schema '{1}', which is not yet supported via USD Interchange."),
				FText::FromString(NodeUid),
				FText::FromString(GetTargetSchemaName())
			));
			return false;
		}

		UInterchangeAudioSoundWaveNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeAudioSoundWaveNode>(
			*UsdContext.GetNodeContainer(),
			NodeUid,
			NodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, Prim.GetPrimPath().GetString());

		static const FString FilePathToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->filePath);
		UE::FUsdAttribute FilePathAttribute = Prim.GetAttribute(*FilePathToken);
		const FString ResolvedAudioFilepath = UsdUtils::GetResolvedAssetPath(FilePathAttribute);
		AssetNode->SetPayloadKey(ResolvedAudioFilepath);

		// TODO: @vjavdekar - Get all the attributes from the media spatial audio
		// and add those to AssetNode.

		return true;
	}

	bool FSpatialAudioSchemaHandler::OnGetAudioPayloadData(
		const FString& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FInterchangeAudioPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSchemaHandler::OnGetAudioPayloadData)

		InOutPayloadData = FInterchangeAudioPayloadDataUtils::GetAudioPayloadFromSourceFileKey(PayloadKey);
		return InOutPayloadData.IsSet();
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
