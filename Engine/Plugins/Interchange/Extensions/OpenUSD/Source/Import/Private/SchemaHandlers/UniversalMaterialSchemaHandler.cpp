// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/UniversalMaterialSchemaHandler.h"

#include "InterchangeTexture2DNode.h"
#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDShadeConversion.h"
#include "UsdWrappers/UsdPrim.h"

#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMaterialReferenceNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTranslatorHelper.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Misc/Paths.h"
#include "UDIMUtilities.h"

#define LOCTEXT_NAMESPACE "UniversalMaterialSchemaHandler"

namespace UE::UniversalMaterialSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	FString CreateTextureNode(
		const UE::FUsdPrim& MaterialPrim,
		const UsdToUnreal::FTextureParameterValue& Value,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeBaseNodeContainer& NodeContainer
	)
	{
		FString NodeName;
		FString NodeUid;
		const bool bSuccess = UInterchangeTextureNode::ExtractNodeUidAndNameFromFilePath(Value.TextureFilePath, NodeUid, NodeName);
		if (!bSuccess)
		{
			return {};
		}

		// We can reuse the same texture node for multiple material channels, but only if they require the exact same settings.
		// This means we must encode the final requested texture settings into the NodeUID and deduplicate using those.
		// A texture used as a normal map needs normal map compression for example, and we may have multiple texture sample nodes pointing at
		// the same filepath and claiming different sRGB values (It's not clear whether that would be a correct thing to do but it's something
		// that could be done)
		NodeUid = FString::Printf(TEXT("%s_%d_%d_%d_%d_%d"), *NodeUid, Value.Group, Value.GetSRGBValue(), Value.AddressX, Value.AddressY, Value.bIsUDIM);

		UInterchangeTexture2DNode* Node = GetExistingNode<UInterchangeTexture2DNode>(NodeContainer, NodeUid);
		if (!Node)
		{
			Node = NewObject<UInterchangeTexture2DNode>(&NodeContainer);
			NodeContainer.SetupNode(Node, NodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset);

			// Associate the texture node with the material's prim path so the pregen pipeline can
			// find the owning target via the ancestor walk. If a texture is shared across multiple
			// materials, the first material to create it wins -- fine for pregen since shared
			// textures are typically within the same target asset.
			UE::Interchange::USD::SetPrimPath(*Node, MaterialPrim.GetPrimPath().GetString());
		}

		Node->SetPayLoadKey(EncodeTexturePayloadKey(Value));

		static_assert((int)TextureAddress::TA_Wrap == (int)EInterchangeTextureWrapMode::Wrap);
		static_assert((int)TextureAddress::TA_Clamp == (int)EInterchangeTextureWrapMode::Clamp);
		static_assert((int)TextureAddress::TA_Mirror == (int)EInterchangeTextureWrapMode::Mirror);
		Node->SetCustomWrapU((EInterchangeTextureWrapMode)Value.AddressX);
		Node->SetCustomWrapV((EInterchangeTextureWrapMode)Value.AddressY);

		Node->SetCustomSRGB(Value.GetSRGBValue());

		// Provide the other UDIM tiles
		//
		// Note: There is an bImportUDIM option on UInterchangeGenericTexturePipeline that is exclusively used within
		// UInterchangeGenericTexturePipeline::HandleCreationOfTextureFactoryNode in order to essentially do the exact same
		// thing as we do here. In theory, we shouldn't need to do this then, and in fact it is a bit bad to do so because
		// we will always parse these UDIMs whether the option is enabled or disabled. The issue however is that (as of the
		// time of this writing) UInterchangeGenericTexturePipeline::HandleCreationOfTextureFactoryNode is hard-coded to expect
		// the texture payload key to be just the texture file path. We can't do that, because we need to also encode
		// the texture compression settings onto payload key...
		//
		// All of that is to say that everything will actually work fine, but if you uncheck "bImportUDIM" on the import options
		// you will still get UDIMs (for now).
		if (Value.bIsUDIM)
		{
			TMap<int32, FString> TileIndexToPath = UE::TextureUtilitiesCommon::GetUDIMBlocksFromSourceFile(
				Value.TextureFilePath,
				UE::TextureUtilitiesCommon::DefaultUdimRegexPattern
			);
			Node->SetSourceBlocks(MoveTemp(TileIndexToPath));
		}

		AccumulatedInfo.PrimAssetNodes.Add(Node);

		return NodeUid;
	}

	// We use this visitor to set UsdToUnreal::FParameterValue TVariant values onto UInterchangeMaterialInstanceNode.
	//
	// For now we only set attributes meant to be parsed as material instance parameters.
	// If/whenever we want to support generating full material shader graphs from USD, we likely don't want to just fill out inputs
	// into a rigid material function structures based on the shading model like the GLTF translator does, as USD materials can have
	// custom shader graphs themselves. We'd either need to truly generate arbitrary interchange shader graphs here to be useful, or
	// to delegate this work to MaterialX somehow (c.f. MaterialX materials baked into USD shader graphs)
	struct FMaterialInstanceParameterValueVisitor
	{
		FMaterialInstanceParameterValueVisitor(
			const UE::FUsdPrim& Prim,
			UInterchangeMaterialInstanceNode& InMaterialNode,
			const TMap<FString, int32>& PrimvarToUVIndex,
			FHandlerAccumulatedInfo& InAccumulatedInfo,
			UInterchangeBaseNodeContainer& InNodeContainer
		)
			: Prim(Prim)
			, MaterialNode(InMaterialNode)
			, PrimvarToUVIndex(PrimvarToUVIndex)
			, AccumulatedInfo(InAccumulatedInfo)
			, NodeContainer(InNodeContainer)
		{
		}

		void EnableTextureForChannel(bool bEnable) const
		{
			MaterialNode.AddScalarParameterValue(UseTextureParameterPrefix + *BaseParameterName + UseTextureParameterSuffix, bEnable ? 1.0f : 0.0f);
		}

		void operator()(const float Value) const
		{
			MaterialNode.AddScalarParameterValue(*BaseParameterName, Value);
			EnableTextureForChannel(false);
		}

		void operator()(const FVector& Value) const
		{
			MaterialNode.AddVectorParameterValue(*BaseParameterName, FLinearColor{Value});
			EnableTextureForChannel(false);
		}

		void operator()(const UsdToUnreal::FTextureParameterValue& Value) const
		{
			const FString TextureUid = CreateTextureNode(Prim, Value, AccumulatedInfo, NodeContainer);
			if (TextureUid.IsEmpty())
			{
				return;
			}

			// Actual texture assignment
			MaterialNode.AddTextureParameterValue(	  //
				FString::Printf(TEXT("%sTexture"), **BaseParameterName),
				TextureUid
			);
			EnableTextureForChannel(true);

			// UV transform
			FLinearColor ScaleAndTranslation = FLinearColor{
				Value.UVScale.GetVector()[0],
				Value.UVScale.GetVector()[1],
				Value.UVTranslation[0],
				Value.UVTranslation[1]
			};
			MaterialNode.AddVectorParameterValue(FString::Printf(TEXT("%sScaleTranslation"), **BaseParameterName), ScaleAndTranslation);
			MaterialNode.AddScalarParameterValue(	 //
				FString::Printf(TEXT("%sRotation"), **BaseParameterName),
				Value.UVRotation
			);

			// UV index
			if (const int32* FoundIndex = PrimvarToUVIndex.Find(Value.Primvar))
			{
				MaterialNode.AddScalarParameterValue(	 //
					*BaseParameterName + UE::Interchange::USD::UVIndexParameterSuffix,
					*FoundIndex
				);
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MissingPrimvar",
						"Failed to find primvar '{0}' when setting material parameter '{1}' on material '{2}'. Available primvars and UV indices: {3}. Is your UsdUVTexture Shader missing the 'inputs:st' attribute? (It specifies which UV set to sample the texture with)"
					),
					FText::FromString(Value.Primvar),
					FText::FromString(*BaseParameterName),
					FText::FromString(Prim.GetPrimPath().GetString()),
					FText::FromString(UsdUtils::StringifyMap(PrimvarToUVIndex))
				));
			}

			// Component mask (which channel of the texture to use)
			FLinearColor ComponentMask = FLinearColor::Black;
			switch (Value.OutputIndex)
			{
				case 0:	   // RGB
					ComponentMask = FLinearColor{1.f, 1.f, 1.f, 0.f};
					break;
				case 1:	   // R
					ComponentMask = FLinearColor{1.f, 0.f, 0.f, 0.f};
					break;
				case 2:	   // G
					ComponentMask = FLinearColor{0.f, 1.f, 0.f, 0.f};
					break;
				case 3:	   // B
					ComponentMask = FLinearColor{0.f, 0.f, 1.f, 0.f};
					break;
				case 4:	   // A
					ComponentMask = FLinearColor{0.f, 0.f, 0.f, 1.f};
					break;
			}
			MaterialNode.AddVectorParameterValue(FString::Printf(TEXT("%sTextureComponent"), **BaseParameterName), ComponentMask);
		}

		void operator()(const UsdToUnreal::FPrimvarReaderParameterValue& Value) const
		{
			MaterialNode.AddVectorParameterValue(*BaseParameterName, FLinearColor{Value.FallbackValue});

			if (Value.PrimvarName == TEXT("displayColor"))
			{
				MaterialNode.AddScalarParameterValue(TEXT("UseVertexColorForBaseColor"), 1.0f);
			}
		}

		void operator()(const bool Value) const
		{
			// Actual booleans are only meant for static switches on Interchange
			MaterialNode.AddScalarParameterValue(*BaseParameterName, static_cast<float>(Value));
		}

		void operator()(const int32 Value) const
		{
			// integers are treated as scalars
			MaterialNode.AddScalarParameterValue(*BaseParameterName, static_cast<float>(Value));
		}

	public:
		const UE::FUsdPrim& Prim;
		UInterchangeMaterialInstanceNode& MaterialNode;
		const TMap<FString, int32>& PrimvarToUVIndex;
		const FString* BaseParameterName = nullptr;
		FHandlerAccumulatedInfo& AccumulatedInfo;
		UInterchangeBaseNodeContainer& NodeContainer;
	};

	void ConvertMaterialPrim(
		const UE::FUsdPrim& Prim,
		UInterchangeMaterialInstanceNode* MaterialNode,
		const TArray<FString>& RenderContexts,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMaterialPrim)

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();

		// Try parsing the material with all the render contexts we support, in order.
		// Even if we find nothing that works we must continue however, so as to finish setting up our
		// MaterialNode with a valid (default) CustomParent at the bottom of this function.
		// Otherwise it's possible we end up producing no valid factory nodes for this USD material
		UsdToUnreal::FUsdPreviewSurfaceMaterialData MaterialData;
		for (const FString& RenderContext : RenderContexts)
		{
			UsdToUnreal::FUsdPreviewSurfaceMaterialData MaterialDataForRenderContext;
			if (UsdToUnreal::ConvertMaterial(Prim, MaterialDataForRenderContext, *RenderContext))
			{
				MaterialData = MaterialDataForRenderContext;
				break;
			}
		}

		// Set all the parameter values to the interchange node
		bool bNeedsVTParent = false;
		FMaterialInstanceParameterValueVisitor Visitor{Prim, *MaterialNode, MaterialData.PrimvarToUVIndex, AccumulatedInfo, *NodeContainer};
		for (TPair<FString, UsdToUnreal::FParameterValue>& Pair : MaterialData.Parameters)
		{
			Visitor.BaseParameterName = &Pair.Key;
			Visit(Visitor, Pair.Value);

			// Also simultaneously check if any of these parameters wants to be an UDIM texture so that we can use the VT reference material later
			if (!bNeedsVTParent)
			{
				if (UsdToUnreal::FTextureParameterValue* TextureParameter = Pair.Value.TryGet<UsdToUnreal::FTextureParameterValue>())
				{
					if (TextureParameter->bIsUDIM)
					{
						bNeedsVTParent = true;
					}
				}
			}
		}

		// Also set our parameter to uv index mapping as-is also as custom attributes, so that the USD Pipeline can make
		// primvar-compatible materials
		{
			// Parameter to primvar
			for (const TPair<FString, UsdToUnreal::FParameterValue>& Parameter : MaterialData.Parameters)
			{
				if (const UsdToUnreal::FTextureParameterValue* TextureParameterValue = Parameter.Value.TryGet<UsdToUnreal::FTextureParameterValue>())
				{
					const FString& MaterialParameter = Parameter.Key;
					const FString& Primvar = TextureParameterValue->Primvar;

					MaterialNode->AddStringAttribute(UE::Interchange::USD::ParameterToPrimvarAttributePrefix + MaterialParameter, Primvar);
				}
			}

			// Primvar to UVIndex
			for (const TPair<FString, int32>& Pair : MaterialData.PrimvarToUVIndex)
			{
				const FString& Primvar = Pair.Key;
				int32 UVIndex = Pair.Value;

				MaterialNode->AddInt32Attribute(UE::Interchange::USD::PrimvarUVIndexAttributePrefix + Primvar, UVIndex);
			}

			// Let the pipeline know that it should process this node and handle these attributes we just added
			if (MaterialData.PrimvarToUVIndex.Num() > 0 || MaterialData.Parameters.Num() > 0)
			{
				MaterialNode->AddBooleanAttribute(UE::Interchange::USD::ParseMaterialIdentifier, true);
			}
		}

		EUsdReferenceMaterialProperties Properties = EUsdReferenceMaterialProperties::None;
		if (UsdUtils::IsMaterialTranslucent(MaterialData))
		{
			Properties |= EUsdReferenceMaterialProperties::Translucent;
		}
		if (bNeedsVTParent)
		{
			// TODO: Proper VT texture support (we'd need to know the texture resolution at this point, and we haven't parsed them yet...).
			// The way it currently works on Interchange is that the factory will create a VT or nonVT version of the texture to match the
			// material parameter slot. Since we'll currently never set the VT reference material, it essentially means it will always
			// downgrade our VT textures to non-VT.
			// The only exception is how we upgrade the reference material to VT in case we have any UDIM textures a few lines above,
			// as those are trivial to check for (we don't have to actually load the textures to do it)
			Properties |= EUsdReferenceMaterialProperties::VT;
		}

		MaterialNode->AddInt32Attribute(UsdReferenceMaterialProperties, uint8(Properties));
		FSoftObjectPath ParentMaterial = UsdUnreal::MaterialUtils::GetReferencePreviewSurfaceMaterial(Properties);
		if (ParentMaterial.IsValid())
		{
			MaterialNode->SetCustomParent(ParentMaterial.GetAssetPathString());
		}
	}
}	 // namespace UE::UniversalMaterialSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FUniversalMaterialSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("UniversalMaterialHandler");
		return HandlerName;
	}

	const TArray<FString>& FUniversalMaterialSchemaHandler::GetDefaultRenderContexts() const
	{
		// The empty string corresponds to the 'universal' render context. When displaying on the UI, we'll swap this empty
		// string to/from the actual 'universal' string
		const static TArray<FString> RenderContexts{TEXT("")};
		return RenderContexts;
	}

	bool FUniversalMaterialSchemaHandler::AllowCustomRenderContexts() const
	{
		return true;
	}

	TOptional<bool> FUniversalMaterialSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FUniversalMaterialSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FUniversalMaterialSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUniversalMaterialSchemaHandler::OnTranslate)

		using namespace UE::UniversalMaterialSchemaHandler::Private;

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return {};
		}

		// For the material schema handlers, we only touch the asset if no other handler has produced a material node yet.
		// This helps manage the separate material handlers per render context, without having them partially overwrite the
		// node with render-context-specific data over and over
		const static TSet<UClass*> MaterialLikeClasses = {
			UInterchangeMaterialInstanceNode::StaticClass(),
			UInterchangeShaderGraphNode::StaticClass(),
			UInterchangeMaterialReferenceNode::StaticClass()
		};
		for (const UInterchangeBaseNode* AssetNode : AccumulatedInfo.PrimAssetNodes)
		{
			for (const UClass* MaterialClass : MaterialLikeClasses)
			{
				if (AssetNode->IsA(MaterialClass))
				{
					return false;
				}
			}
		}

		FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, MaterialPrefix);
		FString NewNodeName{Prim.GetName().ToString()};
		UInterchangeMaterialInstanceNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeMaterialInstanceNode>(
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, Prim.GetPrimPath().GetString());

		const TArray<FString>& RenderContexts = GetCustomRenderContexts();
		ConvertMaterialPrim(Prim, AssetNode, RenderContexts, AccumulatedInfo, UsdContext);

		return true;
	}

	bool FUniversalMaterialSchemaHandler::OnGetTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportImage>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUniversalMaterialSchemaHandler::OnGetTexturePayloadData)

		using namespace UE::InterchangeUsdTranslator::Private;

		FString FilePath;
		TextureGroup TextureGroup;

		bool bDecoded = DecodeTexturePayloadKey(PayloadKey, FilePath, TextureGroup);
		if (!bDecoded)
		{
			return false;
		}

		if (!UsdContext.USDZFilePath.IsEmpty())
		{
			AlternateTexturePath = UsdContext.USDZFilePath;
		}
		else
		{
			AlternateTexturePath = FilePath;
		}

		// Defer back to another translator to actually parse the texture raw data
		//
		// Note: For DomeLights even though we produce texture cube nodes, we'll rely on a common texture payload here (i.e. not sliced),
		// as Unreal handles only latlong / equirectangular, single slice .HDRs anyway. Via the .dds format you can provide sliced payloads
		// for texture cubes, but we do not handle that via USD for now
		UE::Interchange::Private::FScopedTranslator ScopedTranslator{
			FilePath,
			UsdContext.GetResultsContainer(),
			UsdContext.GetTranslator()->AnalyticsHandler
		};
		const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();
		if (TextureTranslator)
		{
			// The texture translators don't use the payload key, and read the texture directly from the SourceData's file path
			const FString UnusedPayloadKey = {};
			InOutPayloadData = TextureTranslator->GetTexturePayloadData(UnusedPayloadKey, AlternateTexturePath);

			if (InOutPayloadData.IsSet())
			{
				// The TextureTranslator doesn't know how we plan on using this texture: If we know it's meant to be a normal map,
				// then override the compression settings here. Leave the rest untouched as the TextureTranslator is going to set
				// that according to the file itself
				if (TextureGroup == TEXTUREGROUP_WorldNormalMap)
				{
					InOutPayloadData->CompressionSettings = TC_Normalmap;
				}

				return true;
			}
		}

		return false;
	}

	bool FUniversalMaterialSchemaHandler::OnGetBlockedTexturePayloadData(
		const FString& PayloadKey,
		TOptional<FString>& AlternateTexturePath,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FImportBlockedImage>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUniversalMaterialSchemaHandler::OnGetBlockedTexturePayloadData)

		using namespace UE::InterchangeUsdTranslator::Private;

		FString FilePath;
		TextureGroup TextureGroup;
		bool bDecoded = DecodeTexturePayloadKey(PayloadKey, FilePath, TextureGroup);
		if (!bDecoded)
		{
			return false;
		}

		if (!UsdContext.USDZFilePath.IsEmpty())
		{
			AlternateTexturePath = UsdContext.USDZFilePath;
		}
		else
		{
			AlternateTexturePath = FilePath;
		}

		// Collect all the UDIM tile filepaths similar to this current tile. If we've been asked to translate
		// a blocked texture then we must have some
		TMap<int32, FString> TileIndexToPath = UE::TextureUtilitiesCommon::GetUDIMBlocksFromSourceFile(
			FilePath,
			UE::TextureUtilitiesCommon::DefaultUdimRegexPattern
		);
		if (!ensure(TileIndexToPath.Num() > 0))
		{
			return false;
		}

		bool bInitializedBlockData = false;

		UE::Interchange::FImportBlockedImage& BlockData = InOutPayloadData.Emplace();

		TArray<UE::Interchange::FImportImage> TileImages;
		TileImages.Reserve(TileIndexToPath.Num());

		for (const TPair<int32, FString>& TileIndexAndPath : TileIndexToPath)
		{
			int32 UdimTile = TileIndexAndPath.Key;
			const FString& TileFilePath = TileIndexAndPath.Value;

			int32 BlockX = INDEX_NONE;
			int32 BlockY = INDEX_NONE;
			UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(UdimTile, BlockX, BlockY);
			if (BlockX == INDEX_NONE || BlockY == INDEX_NONE)
			{
				continue;
			}

			// Find another translator that actually supports that filetype to handle the texture
			UE::Interchange::Private::FScopedTranslator ScopedTranslator(
				TileFilePath,
				UsdContext.GetResultsContainer(),
				UsdContext.GetTranslator()->AnalyticsHandler
			);
			const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>(
			);
			if (!ensure(TextureTranslator))
			{
				continue;
			}

			// Invoke the translator to actually load the texture and parse it
			const FString UnusedPayloadKey = {};
			TOptional<UE::Interchange::FImportImage> TexturePayloadData;
			TexturePayloadData = TextureTranslator->GetTexturePayloadData(UnusedPayloadKey, AlternateTexturePath);
			if (!TexturePayloadData)
			{
				continue;
			}
			const UE::Interchange::FImportImage& Image = TileImages.Emplace_GetRef(MoveTemp(TexturePayloadData.GetValue()));
			TexturePayloadData.Reset();

			// Initialize the settings on the BlockData itself based on the first image we parse
			if (!bInitializedBlockData)
			{
				bInitializedBlockData = true;

				BlockData.Format = Image.Format;
				BlockData.CompressionSettings = TextureGroup == TEXTUREGROUP_WorldNormalMap ? TC_Normalmap : TC_Default;
				BlockData.bSRGB = Image.bSRGB;
				BlockData.MipGenSettings = Image.MipGenSettings;
			}

			// Prepare the BlockData to receive this image data (later)
			BlockData.InitBlockFromImage(BlockX, BlockY, Image);
		}

		// Move all of the FImportImage buffers into the BlockData itself
		BlockData.MigrateDataFromImagesToRawData(TileImages);

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
