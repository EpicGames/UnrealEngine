// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/UnrealMaterialSchemaHandler.h"

#include "InterchangeTexture2DNode.h"
#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDConversionUtils.h"
#include "USDShadeConversion.h"
#include "UsdWrappers/UsdPrim.h"

#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMaterialReferenceNode.h"
#include "InterchangeShaderGraphNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

namespace UE::Interchange::USD
{
	const FString& FUnrealMaterialSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("UnrealMaterialHandler");
		return HandlerName;
	}

	const TArray<FString>& FUnrealMaterialSchemaHandler::GetDefaultRenderContexts() const
	{
		const static TArray<FString> RenderContexts{*UnrealIdentifiers::UnrealRenderContext.ToString()};
		return RenderContexts;
	}

	bool FUnrealMaterialSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealMaterialSchemaHandler::OnTranslate)

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return {};
		}

		// For the material schema handlers, we only touch the asset if no other handler has produced a material node yet.
		// This helps manage the separate material handlers per render context, without having them partially overwrite the
		// node with render-context-specific data over and over
		const static TSet<UClass*> MaterialLikeClasses = {
			UInterchangeMaterialInstanceNode::StaticClass(),
			UInterchangeShaderGraphNode::StaticClass(),
			UInterchangeMaterialReferenceNode::StaticClass()
		};
		for (const UInterchangeBaseNode* AssetNode : AccumulatedInfo.PrimAssetNodes)
		{
			for (const UClass* MaterialClass : MaterialLikeClasses)
			{
				if (AssetNode->IsA(MaterialClass))
				{
					return false;
				}
			}
		}

		FString NewNodeUid = UsdContext.MakeAssetNodeUid(Prim, MaterialPrefix);
		FString NewNodeName{Prim.GetName().ToString()};
		UInterchangeMaterialReferenceNode* AssetNode = AccumulatedInfo.GetOrCreateAssetNode<UInterchangeMaterialReferenceNode>(
			*UsdContext.GetNodeContainer(),
			NewNodeUid,
			NewNodeName
		);
		if (!AssetNode)
		{
			return false;
		}
		UE::Interchange::USD::SetPrimPath(*AssetNode, Prim.GetPrimPath().GetString());

		// If this material has an unreal surface output and we're in the unreal render context, just emit a material reference,
		// as we never want this to become a UMaterial / UMaterialInstance anyway.
		//
		// We could just early out here completely and not emit anything, as we also emit the material reference node on-demand, whenever
		// we parse an actual material assignment from a Mesh. The user may have custom pipelines that expect to find these though,
		// even if no mesh is actually using the materials
		TOptional<FString> UnrealContentPath = UsdUtils::GetUnrealSurfaceOutput(Prim);
		if (UnrealContentPath.IsSet())
		{
			AssetNode->SetCustomContentPath(UnrealContentPath.GetValue());
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
