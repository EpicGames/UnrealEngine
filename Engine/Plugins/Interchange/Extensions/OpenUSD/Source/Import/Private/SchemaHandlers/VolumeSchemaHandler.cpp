// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/VolumeSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerRegistry.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "USDErrorUtils.h"
#include "USDPrimConversion.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeVolumeNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Volume/InterchangeVolumeDefinitions.h"
#include "Volume/InterchangeVolumePayloadInterface.h"
#include "Volume/InterchangeVolumePayloadKey.h"
#include "Volume/InterchangeVolumeTranslatorSettings.h"

#include "Async/ParallelFor.h"
#include "Components/HeterogeneousVolumeComponent.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "VolumeSchemaHandler"

namespace UE::VolVolumeSchemaHandler::Private
{
	using namespace UE::Interchange::USD;

	// Adds a numbered suffix (if needed) to NodeUid to make sure there is nothing with that ID within NodeContainer.
	// Does not add the ID to the NodeContainer.
	void MakeNodeUidUniqueInContainer(FString& NodeUid, const UInterchangeBaseNodeContainer& NodeContainer)
	{
		if (!NodeContainer.IsNodeUidValid(NodeUid) || NodeUid == UInterchangeBaseNode::InvalidNodeUid())
		{
			return;
		}

		int32 Suffix = 0;
		FString Result;
		do
		{
			Result = FString::Printf(TEXT("%s_%d"), *NodeUid, Suffix++);
		} while (NodeContainer.IsNodeUidValid(Result));

		NodeUid = Result;
	}

	// Some of the volume prim info is meant for the HeterogeneousVolume (HV) actor and the volumetric material,
	// so we need to add it to the scene node (it's possible separate HV actors with different values for these
	// end up sharing identical volume nodes)
	void AddVolumeSceneNodeAttributes(
		const UE::FUsdPrim& Prim,
		UInterchangeSceneNode* SceneNode,
		TSet<UInterchangeVolumeNode*> VolumeNodes,
		const FString& VolumeMaterialInstanceUid,
		bool bNeedsAnimationTrack,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddVolumeSceneNodeAttributes)

		if (!SceneNode)
		{
			return;
		}
		const FString& PrimPath = Prim.GetPrimPath().GetString();

		// Target the volume nodes from the scene node to make it easy to find everything we need on the USD Pipeline
		SceneNode->SetCustomAssetInstanceUid((*VolumeNodes.CreateIterator())->GetUniqueID());
		for (UInterchangeVolumeNode* VolumeNode : VolumeNodes)
		{
			SceneNode->AddTargetNodeUid(VolumeNode->GetUniqueID());
		}

		// Set our volumetric material as a "material override" directly on the scene node, which the USD Pipeline will also use
		if (!VolumeMaterialInstanceUid.IsEmpty())
		{
			SceneNode->SetSlotMaterialDependencyUid(UE::Interchange::Volume::VolumetricMaterial, VolumeMaterialInstanceUid);
		}

		if (bNeedsAnimationTrack)
		{
			// Ideally we'd write some step curves, but Interchange doesn't support float step curves
			const EInterchangeAnimationPayLoadType PayloadType = EInterchangeAnimationPayLoadType::CURVE;
			const EInterchangePropertyTracks TrackType = EInterchangePropertyTracks::HeterogeneousVolumeFrame;
			const static FName UEPropertyName = GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame);

			AddPropertyAnimationNode(PrimPath, UEPropertyName, TrackType, PayloadType, AccumulatedInfo, UsdContext);
		}
	}

	void AddVolumeCustomAttributesToNode(const UsdUtils::FVolumePrimInfo& VolumePrimInfo, UInterchangeVolumeNode* VolumeNode)
	{
		// Convert from the {'velocity': {'AttributesA.R': 'X', 'AttributesA.G': 'Y', 'AttributesA.B': 'Z'}} style of mapping
		// from grid info into {"AttributesA.X": "velocity_0", "AttributesA.G": "velocity_1", "AttributesA.B": "velocity_2"}
		// mapping into the VolumeNode custom attributes
		for (const TPair<FString, TMap<FString, FString>>& GridPair : VolumePrimInfo.GridNameToChannelComponentMapping)
		{
			const FString& GridName = GridPair.Key;	   // "density", "temperature", etc.

			const TMap<FString, FString>& AttributesChannelToGridChannel = GridPair.Value;
			for (const TPair<FString, FString>& AssignmentPair : AttributesChannelToGridChannel)
			{
				const FString& AttributeChannel = AssignmentPair.Key;	 // "AttributesA.R", "AttributesB.B", etc.
				const FString& GridChannel = AssignmentPair.Value;		 // "X", "Y", "Z", etc.

				const static TMap<FString, FString> AttributeChannelToAttributeKey{
					{TEXT("AttributesA.R"), SparseVolumeTexture::AttributesAChannelR},
					{TEXT("AttributesA.G"), SparseVolumeTexture::AttributesAChannelG},
					{TEXT("AttributesA.B"), SparseVolumeTexture::AttributesAChannelB},
					{TEXT("AttributesA.A"), SparseVolumeTexture::AttributesAChannelA},
					{TEXT("AttributesB.R"), SparseVolumeTexture::AttributesBChannelR},
					{TEXT("AttributesB.G"), SparseVolumeTexture::AttributesBChannelG},
					{TEXT("AttributesB.B"), SparseVolumeTexture::AttributesBChannelB},
					{TEXT("AttributesB.A"), SparseVolumeTexture::AttributesBChannelA},
				};

				const static TMap<FString, FString> GridChannelToComponentIndex{
					{TEXT("X"), TEXT("0")},
					{TEXT("Y"), TEXT("1")},
					{TEXT("Z"), TEXT("2")},
					{TEXT("W"), TEXT("3")},
					{TEXT("R"), TEXT("0")},
					{TEXT("G"), TEXT("1")},
					{TEXT("B"), TEXT("2")},
					{TEXT("A"), TEXT("3")},
				};

				const FString* FoundAttributeKey = AttributeChannelToAttributeKey.Find(AttributeChannel);
				if (!FoundAttributeKey)
				{
					USD_LOG_WARNING(TEXT("Failing to parse unreal:SVT:mappedAttributeChannels value '%s'"), *AttributeChannel);
					continue;
				}

				const FString* FoundComponentIndex = GridChannelToComponentIndex.Find(GridChannel);
				if (!FoundComponentIndex)
				{
					USD_LOG_WARNING(TEXT("Failing to parse unreal:SVT:mappedGridComponents value '%s'"), *GridChannel);
					continue;
				}

				VolumeNode->AddStringAttribute(
					*FoundAttributeKey,
					GridName + UE::Interchange::Volume::GridNameAndComponentIndexSeparator + *FoundComponentIndex
				);
			}
		}

		// Convert texture format
		static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Unorm8 == (int)EInterchangeSparseVolumeTextureFormat::Unorm8);
		static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Float16 == (int)EInterchangeSparseVolumeTextureFormat::Float16);
		static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Float32 == (int)EInterchangeSparseVolumeTextureFormat::Float32);
		if (VolumePrimInfo.AttributesAFormat.IsSet())
		{
			VolumeNode->AddInt32Attribute(SparseVolumeTexture::AttributesAFormat, static_cast<int32>(VolumePrimInfo.AttributesAFormat.GetValue()));
		}
		if (VolumePrimInfo.AttributesBFormat.IsSet())
		{
			VolumeNode->AddInt32Attribute(SparseVolumeTexture::AttributesBFormat, static_cast<int32>(VolumePrimInfo.AttributesBFormat.GetValue()));
		}
	}

	UInterchangeMaterialInstanceNode* CreateVolumetricMaterialInstanceNode(
		const UE::FUsdPrim& VolumePrim,
		const UsdToUnreal::FUsdMeshConversionOptions& ConversionOptions,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeBaseNodeContainer& NodeContainer
	)
	{
		FString ParentMaterialPath;

		// Priority 1: Explicit material assignment on the Volume prim (Unreal materials).
		{
			// Don't pay too much attention to the render contexts right now, because we have no idea what our
			// material schema handlers will do with this material
			double TimeCode = UsdUtils::GetDefaultTimeCode();	 // Not relevant as material bindings can't be time sampled
			bool bProvideMaterialIndices = false;
			UsdUtils::FUsdPrimMaterialAssignmentInfo Assignments = UsdUtils::GetPrimMaterialAssignments(
				VolumePrim,
				TimeCode,
				bProvideMaterialIndices,
				UnrealIdentifiers::Unreal,	  // Unreal materials should always be on the unreal render context
				ConversionOptions.MaterialPurpose,
				ConversionOptions.bForceCheckUnrealMaterialAttribute,
				ConversionOptions.bRequireMaterialsHaveProvidedRenderContext
			);

			for (const UsdUtils::FUsdPrimMaterialSlot& Slot : Assignments.Slots)
			{
				if (Slot.AssignmentType == UsdUtils::EPrimAssignmentType::UnrealMaterial)
				{
					ParentMaterialPath = Slot.MaterialSource;
					break;
				}
			}
		}

		// Priority 2: The USD default volumetric material on the UsdProjectSettings.
		bool bIsFallback = false;
		if (ParentMaterialPath.IsEmpty())
		{
			if (const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>())
			{
				bIsFallback = true;
				ParentMaterialPath = ProjectSettings->ReferenceDefaultSVTMaterial.ToString();
			}
		}

		// Priority 3: Hard-coded fallback volumetric material that ships with the engine.
		if (ParentMaterialPath.IsEmpty())
		{
			bIsFallback = true;
			ParentMaterialPath = TEXT("/Engine/EngineMaterials/SparseVolumeMaterial.SparseVolumeMaterial");
		}

		FString MaterialDisplayLabel = bIsFallback ? TEXT("USDVolumetricFallbackMaterial") : FPaths::GetBaseFilename(ParentMaterialPath);
		FString MaterialNodeUid = MakeNodeUid(MaterialPrefix + MaterialDisplayLabel);
		MakeNodeUidUniqueInContainer(MaterialNodeUid, NodeContainer);

		// We'll always spawn a new material instance for each volume prim. Realistically a stage is only going to have a handful
		// of volumes at most, and material instances should be pretty cheap. This should be a more predictable result for the user,
		// and it prevents us from needing some bespoke code to reuse these material instance nodes depending on their volume assignment
		UInterchangeMaterialInstanceNode* MaterialInstance = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
		NodeContainer.SetupNode(MaterialInstance, MaterialNodeUid, MaterialDisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
		MaterialInstance->SetCustomParent(ParentMaterialPath);
		AccumulatedInfo.PrimAssetNodes.Add(MaterialInstance);

		return MaterialInstance;
	}

	UInterchangeShaderGraphNode* TryCreatingVolumetricShaderGraphNode(const UE::FUsdPrim& VolumePrim, UInterchangeUsdContext& UsdContext)
	{
		// Don't pay too much attention to the render contexts right now, because we have no idea what our
		// material schema handlers will do with this material
		const bool bProvideMaterialIndices = false;
		const pxr::TfToken MtlxToken{"mtlx"};	 // We no longer need to pass this, but there's no reason *not* to pass mtlx here either
		const bool bForceCheckUnrealMaterialAttribute = false;
		UsdUtils::FUsdPrimMaterialAssignmentInfo Assignments = UsdUtils::GetPrimMaterialAssignments(
			VolumePrim,
			UsdUtils::GetDefaultTimeCode(),
			bProvideMaterialIndices,
			MtlxToken,
			UsdContext.CachedMeshConversionOptions.MaterialPurpose,
			bForceCheckUnrealMaterialAttribute,
			UsdContext.CachedMeshConversionOptions.bRequireMaterialsHaveProvidedRenderContext
		);

		for (const UsdUtils::FUsdPrimMaterialSlot& Slot : Assignments.Slots)
		{
			if (Slot.AssignmentType != UsdUtils::EPrimAssignmentType::MaterialPrim)
			{
				continue;
			}

			// MaterialSource here is the material prim path
			UE::FUsdPrim MaterialPrim = UsdContext.GetUsdStage().GetPrimAtPath(UE::FSdfPath{*Slot.MaterialSource});
			const bool bMaterialXOutput = UsdUtils::HasSurfaceOutput(MaterialPrim, UnrealIdentifiers::MaterialXRenderContext)
										  || UsdUtils::HasVolumeOutput(MaterialPrim, UnrealIdentifiers::MaterialXRenderContext)
										  || UsdUtils::HasDisplacementOutput(MaterialPrim, UnrealIdentifiers::MaterialXRenderContext);
			if (!bMaterialXOutput)
			{
				continue;
			}

			UInterchangeUsdTranslatorSettings* Settings = UsdContext.GetTranslatorSettings();
			if (!Settings)
			{
				continue;
			}

			// Check if we have a handler capable of parsing MaterialX
			// This is not a perfect check, but it should hopefully be good enough for now
			bool bMaterialXHandler = false;
			const TArray<FSchemaHandlerEntry>* HandlerEntries = &FSchemaHandlerRegistry::RegisteredHandlerEntries;
			if (Settings && Settings->CustomHandlerEntries.Num() > 0)
			{
				HandlerEntries = &Settings->CustomHandlerEntries;
			}
			if (HandlerEntries)
			{
				for (const FSchemaHandlerEntry& Entry : *HandlerEntries)
				{
					const static FString MtlxString = *UnrealIdentifiers::MaterialXRenderContext.ToString();
					if (Entry.bEnabled && Entry.DefaultRenderContexts.Contains(MtlxString))
					{
						bMaterialXHandler = true;
					}
				}
			}
			if (!bMaterialXHandler)
			{
				continue;
			}

			if (UInterchangeUSDTranslator* CurrentTranslator = UsdContext.GetTranslator())
			{
				FTraversalInfo TraversalInfo;
				if (TOptional<FHandlerAccumulatedInfo> AccumulatedInfoForPrim = CurrentTranslator->TranslatePrim(MaterialPrim, TraversalInfo))
				{
					if (UInterchangeShaderGraphNode* MaterialNode = AccumulatedInfoForPrim->GetAssetNodeOfClass<UInterchangeShaderGraphNode>())
					{
						return MaterialNode;
					}
				}
			}
		}

		return nullptr;
	}

	TSet<UInterchangeVolumeNode*> CreateVolumeNodes(
		const UE::FUsdPrim& Prim,
		TMap<FString, TArray<UsdUtils::FVolumePrimInfo>>& InOutPrimPathToVolumeInfo,
		TMap<FString, TMap<FString, UInterchangeVolumeNode*>>& InOutVolumeFilepathToAnimationIDToNode,
		FString& OutMaterialInstanceUid,
		bool& bOutNeedsFrameTrack,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateVolumeNodes)

		UInterchangeBaseNodeContainer* NodeContainer = UsdContext.GetNodeContainer();
		if (!NodeContainer)
		{
			return {};
		}

		// Each Volume prim can reference multiple FieldAsset prims. Each FieldAsset itself can point to a particular grid within
		// a .vdb file... USD is probably too flexible here, allowing us to reference grids from separate .vdb files in the same Volume prim,
		// or letting us refer to the same grid more than once, etc.
		//
		// Our end goal is to make each .vdb file into a single SparseVolumeTexture, combining all the grids that need to be read from it.
		// We'll do our best to satisfy all the requirements, but emit some warnings if we fail on an edge case. Then we want to spawn a single
		// HeterogeneousVolumeActor for each Volume prim, generate an instance of the right material, and assign all these generated SVTs to it.
		//
		// In Interchange terms, this means that we'll emit a single InterchangeVolumeNode for each .vdb file, but will
		// emit a new UInterchangeVolumeGridNode for each grid reference within that file. Note that USD's flexibility means we may have
		// separate Volume prims all referencing the same shared FieldAsset prim, so we need to presume a InterchangeVolumeNode for this .vdb
		// file has potentially already been created when parsing another Volume prim...
		//
		// Finally, since we may use the same .vdb file in multiple animations, and we want to end up with separate animated SVTs, we need
		// separate factory nodes. If we want to keep the expected mapping of factory node / volume node Uids (just an added "Factory_" prefix)
		// this means we need a separate volume node *per animation*, so we'll also use "AnimationIDs" to differentiate them

		TStrongObjectPtr<UInterchangeTranslatorBase> Translator = nullptr;

		// This is collected by path hash here because for animated SVTs we want to still have a single UsdUtils::FVolumePrimInfo for each
		// group of animated volume frames, since they will become a separate SVT. If we just collected them by filepath we could run into
		// trouble if we had a volume prim with 3 frames starting at "file.vdb", and a separate volume prim that just wants one frame, "frame.vdb"
		TMap<FString, UsdUtils::FVolumePrimInfo> VolumeInfoByFilePathHash = UsdUtils::GetVolumeInfoByFilePathHash(Prim);

		TSet<UInterchangeVolumeNode*> VolumeNodes;
		TMap<FString, FString> VolumeFieldNameToNodeUids;

		bOutNeedsFrameTrack = false;

		// Stash these as we may need this info later when retrieving animation pipelines
		TArray<UsdUtils::FVolumePrimInfo>& CollectedInfoForPrim = InOutPrimPathToVolumeInfo.FindOrAdd(Prim.GetPrimPath().GetString());

		for (const TPair<FString, UsdUtils::FVolumePrimInfo>& Pair : VolumeInfoByFilePathHash)
		{
			const FString& AnimationID = Pair.Key;
			const UsdUtils::FVolumePrimInfo& VolumePrimInfo = Pair.Value;
			CollectedInfoForPrim.Add(VolumePrimInfo);

			TArray<FString> VDBFilePaths;
			VDBFilePaths.Reserve(VolumePrimInfo.TimeSamplePathIndices.Num() + 1);

			// In case we have both timeSamples and different default opinion, add the default opinion as the first frame so that's what it shows
			// on the level. The LevelSequence Frame track will factor this in, and have the LevelSequence only go through the TimeSamplePaths frames
			// though.
			bool bInsertedDefaultOpinion = false;
			if (VolumePrimInfo.TimeSamplePaths.Num() > 0 && VolumePrimInfo.TimeSamplePaths[0] != VolumePrimInfo.SourceVDBFilePath)
			{
				VDBFilePaths.Add(VolumePrimInfo.SourceVDBFilePath);
				bInsertedDefaultOpinion = true;
			}
			// No time samples at all
			else if (VolumePrimInfo.TimeSamplePaths.Num() == 0)
			{
				VDBFilePaths.Add(VolumePrimInfo.SourceVDBFilePath);
			}

			// Add the file paths going through TimeSamplePathIndices because it's possible that GetVolumeInfoByFilePathHash
			// deduplicated volume frames already. It's fine to add duplicate entries to VDBFilePaths though, because we'll check
			// for an existing volume node for that path every time anyway
			for (int32 PathIndex : VolumePrimInfo.TimeSamplePathIndices)
			{
				VDBFilePaths.Add(VolumePrimInfo.TimeSamplePaths[PathIndex]);
			}

			if (VolumePrimInfo.TimeSamplePathTimeCodes.Num() > 0)
			{
				bOutNeedsFrameTrack = true;
			}

			// First volume is special as that is what will "become" the animated factory node if we have animation. We'll also only stash
			// our custom attributes on this first volume node. The filepaths are sorted according to timeSamples, so this is always the first
			// frame of animation, or the default opinion if we have that
			UInterchangeVolumeNode* FirstVolumeNode = nullptr;

			TSet<UInterchangeVolumeNode*> ResetVolumeNodes;
			ResetVolumeNodes.Reserve(VDBFilePaths.Num());

			for (int32 Index = 0; Index < VDBFilePaths.Num(); ++Index)
			{
				const FString& FilePath = VDBFilePaths[Index];

				TMap<FString, UInterchangeVolumeNode*>& AnimationIDToVolumeNode = InOutVolumeFilepathToAnimationIDToNode.FindOrAdd(FilePath);
				UInterchangeVolumeNode* VolumeNode = AnimationIDToVolumeNode.FindRef(AnimationID);

				// Need to translate the volume for this animation ID.
				// Note: It may seem wasteful to translate the same volume more than once in case it is used by multiple animations,
				// but keep in mind that:
				//  - Multiple animations for the same volume frame in the same import will realistically never happen in practice;
				//  - The VDB translator will cache the read file bytes from the first translation;
				//  - "Translating" the volume just involves returning some header information, which should be pretty fast;
				//  - Doing this saves us from having to manually duplicate other volume and grid nodes, and manually patching up
				//    their unique IDs and / or making some sort of mistake;
				if (!VolumeNode)
				{
					UInterchangeSourceData* SourceData = UInterchangeManager::CreateSourceData(FilePath);

					if (!Translator || !Translator->CanImportSourceData(SourceData))
					{
						// Pass a USD context object to the translator, which is a signal that lets the OpenVDB translator
						// be considered, even if its cvar is off (c.f. UInterchangeOpenVDBTranslator::CanImportSourceData)
						SourceData->SetContextObjectByTag(UE::Interchange::USD::USDContextTag, &UsdContext);

						UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
						Translator.Reset(InterchangeManager.GetTranslatorForSourceData(SourceData));
					}
					if (!Translator)
					{
						const bool bIncludeDot = false;
						const FString Extension = FPaths::GetExtension(FilePath, bIncludeDot).ToLower();
						const bool bIsOpenVDB = Extension == TEXT("vdb");

						USD_LOG_USERERROR(FText::Format(
							LOCTEXT("MissingTranslator", "Failed to find a compatible translator for file '{0}'"),
							FText::FromString(FilePath)
						));
						continue;
					}

					if (UInterchangeVolumeTranslatorSettings* Settings = Cast<UInterchangeVolumeTranslatorSettings>(Translator->GetSettings()))
					{
						// We never want to discover new .vdb files via the OpenVDB translator for animations.
						// If we should have animations via USD they will be described on the USD file explicictly
						Settings->bTranslateAdjacentNumberedFiles = false;

						// If the volume prim describes an animation, let's add the same AnimationID to the volume nodes that the translator
						// will output, so that the SVT pipeline groups them up into a single factory node
						Settings->AnimationID = VolumePrimInfo.TimeSamplePathTimeCodes.Num() > 0 ? AnimationID : FString{};
					}

					Translator->SourceData = SourceData;
					Translator->Translate(*NodeContainer);

					// Cache that we used this translator for this filepath. If we keep it, we don't have to open the file again to
					// retrieve the payload. Note: This is likely the same translator we used for all .vdb files during this import
					UsdContext.SubTranslators.Add(FilePath, Translator);

					// Note the slight exploit: We can get non-const access to the translated nodes in this way, which we need
					NodeContainer->BreakableIterateNodesOfType<UInterchangeVolumeNode>(
						[&VolumeNode, &FilePath](const FString& NodeUid, UInterchangeVolumeNode* Node)
						{
							if (NodeUid.Contains(FilePath))
							{
								VolumeNode = Node;
								return true;
							}

							return false;
						}
					);

					if (VolumeNode)
					{
						// The USD shader graph, in case of a volume material, reference the OpenVDBAsset as a primvar
						// In the MaterialX Translator we create a SparseVolumeTextureSampleParameter that references this 'primvar' name, the SparseVolumeTexture asset
						VolumeNode->AddInt32Attribute(Primvar::Number, VolumePrimInfo.VolumeFieldNames.Num());
						for (int32 IndexPrimvar = 0; IndexPrimvar < VolumePrimInfo.VolumeFieldNames.Num(); ++IndexPrimvar)
						{
							VolumeNode->AddStringAttribute(Primvar::ShaderNodeSparseVolumeTextureSample + FString::FromInt(IndexPrimvar), VolumePrimInfo.VolumeFieldNames[IndexPrimvar]);
						}
						AnimationIDToVolumeNode.Add(AnimationID, VolumeNode);
					}
				}

				if (!VolumeNode)
				{
					USD_LOG_USERWARNING(
						FText::Format(LOCTEXT("FailedVolumeNode", "Failed to produce a volume node from file '{0}'"), FText::FromString(FilePath))
					);
					continue;
				}

				// Remove any animation index that the OpenVDB translator (or a translation of another instance of this whole volume prim in the
				// stage) may have set on this volume node, as we want to set the animation indices exclusively during this pass through VDBFilePaths
				if (!ResetVolumeNodes.Contains(VolumeNode))
				{
					ResetVolumeNodes.Add(VolumeNode);

					TArray<int32> ExistingAnimationIndices;
					VolumeNode->GetCustomFrameIndicesInAnimation(ExistingAnimationIndices);
					for (const int32 ExistingIndex : ExistingAnimationIndices)
					{
						VolumeNode->RemoveCustomFrameIndexInAnimation(ExistingIndex);
					}
				}
				VolumeNode->AddCustomFrameIndexInAnimation(Index);

				if (!FirstVolumeNode)
				{
					FirstVolumeNode = VolumeNode;
				}

				VolumeNodes.Add(VolumeNode);
			}

			if (FirstVolumeNode)
			{
				// Collect all of our custom-schema-based assignment info to be used as as custom attributes on the first volume node.
				// The USD Pipeline will handle these, and move them into the factory nodes.
				AddVolumeCustomAttributesToNode(VolumePrimInfo, FirstVolumeNode);

				for (const FString& FieldName : VolumePrimInfo.VolumeFieldNames)
				{
					VolumeFieldNameToNodeUids.Add(FieldName, FirstVolumeNode->GetUniqueID());
				}
			}
		}

		// If we have a material with a MaterialX output and a MaterialX handler is enabled, then let's assume we'll end up parsing
		// this material as a MaterialX shader graph and return that
		if (UInterchangeShaderGraphNode* ShaderGraphMaterial = TryCreatingVolumetricShaderGraphNode(Prim, UsdContext))
		{
			OutMaterialInstanceUid = ShaderGraphMaterial->GetUniqueID();
			return VolumeNodes;
		}

		// Setup a new material instance node for this volume, which will be used by the Heterogeneous Volume Actor we'll also
		// spawn for this volume prim.
		//
		// Note that this is slightly different from other material handling because in here we need a material *instance*, even if
		// we'se using Unreal materials, so we don't use our regular SetSlotMaterialDependencies() call
		//
		// Note that there's only one material slot per actor, but that we really do need some kind of material to be put in there,
		// so that we can at least assign our SVTs somewhere.
		UInterchangeMaterialInstanceNode* MaterialInstance = CreateVolumetricMaterialInstanceNode(
			Prim,
			UsdContext.CachedMeshConversionOptions,
			AccumulatedInfo,
			*NodeContainer
		);

		// Assign SVTs as material parameters on our new material instance
		if (ensure(MaterialInstance))
		{
			OutMaterialInstanceUid = MaterialInstance->GetUniqueID();

			TMultiMap<FString, FString> MaterialParameterToFieldName = UsdUtils::GetVolumeMaterialParameterToFieldNameMap(Prim);

			// Prim doesn't have the attributes specifying an explicit material parameter name to volume mapping (this is probably
			// the more common case)
			if (MaterialParameterToFieldName.Num() == 0)
			{
				// Consider that the field names may match material parameter names. We can't tell if that's the case or not from here,
				// and this may cause us to assign a VolumeUid more than once (as multiple fields may target the same SVT), but the
				// USDPipeline will clean this up
				for (const TPair<FString, FString>& Pair : VolumeFieldNameToNodeUids)
				{
					const FString& FieldName = Pair.Key;
					const FString& VolumeUid = Pair.Value;

					MaterialInstance->AddTextureParameterValue(UE::Interchange::USD::VolumeFieldNameMaterialParameterPrefix + FieldName, VolumeUid);
				}
			}
			// Prim has custom attributes specifying exactly which volume should be assigned to which material parameter
			else
			{
				TMap<FString, FString> ParameterNameToVolume;
				ParameterNameToVolume.Reserve(MaterialParameterToFieldName.Num());

				for (const TPair<FString, FString>& Pair : MaterialParameterToFieldName)
				{
					const FString& ParameterName = Pair.Key;
					const FString& FieldName = Pair.Value;

					const FString* FoundVolumeUid = VolumeFieldNameToNodeUids.Find(FieldName);
					if (!FoundVolumeUid)
					{
						continue;
					}

					// Show a warning in case we have conflicting assignments, because the legacy schema translator also did it
					if (FString* FoundAssignedVolume = ParameterNameToVolume.Find(ParameterName))
					{
						if (*FoundAssignedVolume != *FoundVolumeUid)
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"DifferentTexturesOnSameParameter",
									"Trying to assign different Sparse Volume Textures to the same material parameter '{0}' on the material instantiated for Volume '{1}' and field name '{2}'! Only a single texture can be assigned to a material parameter at a time."
								),
								FText::FromString(ParameterName),
								FText::FromString(*FoundVolumeUid),
								FText::FromString(*FieldName)
							));
						}
						continue;
					}
					ParameterNameToVolume.Add(ParameterName, *FoundVolumeUid);

					MaterialInstance->AddTextureParameterValue(ParameterName, *FoundVolumeUid);
				}
			}
		}

		return VolumeNodes;
	}

	// Volume animations need a special property reader because in USD it's an animation of the file path attribute within
	// the asset prims, while in UE we want a float track flipping through the volume frame indices
	UsdToUnreal::FPropertyTrackReader CreateVolumeTrackReader(
		const FString& PrimPath,
		TMap<FString, TArray<UsdUtils::FVolumePrimInfo>>& PrimPathToVolumeInfo,
		TArray<double>& OutTimeSampleUnion
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateVolumeTrackReader)

		UsdToUnreal::FPropertyTrackReader Result;

		const TArray<UsdUtils::FVolumePrimInfo>* CollectedInfoForPrim = PrimPathToVolumeInfo.Find(PrimPath);
		if (!CollectedInfoForPrim)
		{
			return Result;
		}

		const UsdUtils::FVolumePrimInfo* AnimatedInfo = nullptr;

		for (const UsdUtils::FVolumePrimInfo& Info : *CollectedInfoForPrim)
		{
			if (Info.TimeSamplePathTimeCodes.Num() > 0)
			{
				if (AnimatedInfo != nullptr)
				{
					USD_LOG_USERINFO(FText::Format(
						LOCTEXT(
							"SingleAnimatedSVT",
							"Only one animated SparseVolumeTexture can be driven via LevelSequences for each prim, for now. Prim '{0}' has multiple, so the animation may not be correct."
						),
						FText::FromString(PrimPath)
					));
				}

				AnimatedInfo = &Info;
			}
		}

		if (AnimatedInfo)
		{
			// Detect whether we inserted the default opinion volume as the first frame of the animation
			bool bInsertedDefaultOpinion = AnimatedInfo->TimeSamplePaths.Num() > 0
										   && AnimatedInfo->TimeSamplePaths[0] != AnimatedInfo->SourceVDBFilePath;

			OutTimeSampleUnion = AnimatedInfo->TimeSamplePathTimeCodes;
			Result.FloatReader = [AnimatedInfo, bInsertedDefaultOpinion](double TimeCode) -> float
			{
				int32 FrameIndex = 0;
				for (; FrameIndex + 1 < AnimatedInfo->TimeSamplePathTimeCodes.Num(); ++FrameIndex)
				{
					if (AnimatedInfo->TimeSamplePathTimeCodes[FrameIndex + 1] > TimeCode)
					{
						break;
					}
				}
				FrameIndex = FMath::Clamp(FrameIndex, 0, AnimatedInfo->TimeSamplePathTimeCodes.Num() - 1);

				if (bInsertedDefaultOpinion)
				{
					FrameIndex += 1;
				}

				return static_cast<float>(FrameIndex);
			};
		}

		return Result;
	}

	bool GetPropertyAnimationCurvePayloadData(
		const FString& PayloadKey,
		TMap<FString, TArray<UsdUtils::FVolumePrimInfo>>& PrimPathToVolumeInfo,
		UInterchangeUsdContext& UsdContext,
		Interchange::FAnimationPayloadData& OutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetPropertyAnimationCurvePayloadData)

		FString PrimPath;
		FString UEPropertyNameStr;
		bool bSplit = PayloadKey.Split(TEXT("\\"), &PrimPath, &UEPropertyNameStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!bSplit)
		{
			return false;
		}

		UE::FUsdStage UsdStage = UsdContext.GetUsdStage();
		if (!UsdStage)
		{
			return false;
		}

		UE::FUsdPrim Prim = UsdStage.GetPrimAtPath(UE::FSdfPath{*PrimPath});
		FName UEPropertyName = *UEPropertyNameStr;
		if (!Prim || UEPropertyName == NAME_None)
		{
			return false;
		}

		TArray<double> TimeSampleUnion;
		UsdToUnreal::FPropertyTrackReader Reader;

		if (UEPropertyNameStr == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame))
		{
			Reader = CreateVolumeTrackReader(PrimPath, PrimPathToVolumeInfo, TimeSampleUnion);
		}

		if (Reader.FloatReader)
		{
			return ReadFloats(UsdStage, TimeSampleUnion, Reader.FloatReader, OutPayloadData);
		}

		return false;
	}
}	 // namespace UE::VolVolumeSchemaHandler::Private

namespace UE::Interchange::USD
{
	const FString& FVolumeSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("VolumeHandler");
		return HandlerName;
	}

	const FString& FVolumeSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Volume");
		return SchemaName;
	}

	TOptional<bool> FVolumeSchemaHandler::CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	TOptional<bool> FVolumeSchemaHandler::CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const
	{
		return false;
	}

	bool FVolumeSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVolumeSchemaHandler::OnTranslate)

		using namespace UE::VolVolumeSchemaHandler::Private;

		FString VolumeMaterialInstanceUid;
		bool bNeedsVolumeTrack = false;
		TSet<UInterchangeVolumeNode*> NewVolumeNodes = CreateVolumeNodes(
			Prim,
			PrimPathToVolumeInfo,
			VolumeFilepathToAnimationIDToNode,
			VolumeMaterialInstanceUid,
			bNeedsVolumeTrack,
			AccumulatedInfo,
			UsdContext
		);

		if (NewVolumeNodes.Num() > 0)
		{
			UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);

			// For now we'll only assign the volumes if we're producing actors
			UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase);
			if (!SceneNode)
			{
				return false;
			}

			AddVolumeSceneNodeAttributes(
				Prim,
				SceneNode,
				NewVolumeNodes,
				VolumeMaterialInstanceUid,
				bNeedsVolumeTrack,
				AccumulatedInfo,
				UsdContext
			);

			for (UInterchangeVolumeNode* VolumeNode : NewVolumeNodes)
			{
				AccumulatedInfo.PrimAssetNodes.Add(VolumeNode);
			}

			return true;
		}

		return false;
	}

	bool FVolumeSchemaHandler::OnGetVolumePayloadData(
		const UE::Interchange::FVolumePayloadKey& PayloadKey,
		UInterchangeUsdContext& UsdContext,
		TOptional<UE::Interchange::FVolumePayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVolumeSchemaHandler::OnGetVolumePayloadData)

		using namespace UE::VolVolumeSchemaHandler::Private;

		TStrongObjectPtr<UInterchangeTranslatorBase> ExistingTranslator = UsdContext.SubTranslators.FindRef(PayloadKey.FileName);
		const IInterchangeVolumePayloadInterface* VolumeInterface = Cast<IInterchangeVolumePayloadInterface>(ExistingTranslator.Get());
		if (VolumeInterface)
		{
			InOutPayloadData = VolumeInterface->GetVolumePayloadData(PayloadKey);
			if (InOutPayloadData)
			{
				// Bake our stage coordinate system transform into the volume, just like we do for geometry.
				// We do this here because OpenVDB doesn't really specify a fixed coordinate system, and so we interpret it as just meaning
				// "units" in general. This means we must go through a metersPerUnit / upAxis conversion to get it into UE's world space
				const FUsdStageInfo StageInfo{UsdContext.GetUsdStage()};
				FTransform StageInfoTransform = UsdUtils::GetUsdToUESpaceTransform(StageInfo);
				InOutPayloadData.GetValue().Transform = InOutPayloadData->Transform * StageInfoTransform;
			}
		}

		return InOutPayloadData.IsSet();
	}

	bool FVolumeSchemaHandler::OnGetAnimationPayloadData(
		const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
		UInterchangeUsdContext& UsdContext,
		TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVolumeSchemaHandler::OnGetAnimationPayloadData)

		using namespace UE::VolVolumeSchemaHandler::Private;

		TArray<int32> CurveQueryIndexes;
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadQueries.Num(); ++PayloadIndex)
		{
			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];
			if (PayloadQuery.PayloadKey.Type == EInterchangeAnimationPayLoadType::CURVE)
			{
				CurveQueryIndexes.Add(PayloadIndex);
			}
		}
		if (CurveQueryIndexes.Num() == 0)
		{
			return true;
		}

		TArray<TArray<UE::Interchange::FAnimationPayloadData>> CurveAnimationPayloads;
		auto GetAnimPayloadLambda = [this, &PayloadQueries, &CurveAnimationPayloads, &UsdContext](int32 PayloadIndex)
		{
			if (!PayloadQueries.IsValidIndex(PayloadIndex))
			{
				return;
			}

			const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadIndex];

			// We're fine handling these in isolation (currently GetAnimationPayloadData is called with
			// a single query at a time for these): Emit a separate task for each right away
			FAnimationPayloadData AnimationPayLoadData{PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey};
			if (GetPropertyAnimationCurvePayloadData(PayloadQuery.PayloadKey.UniqueId, PrimPathToVolumeInfo, UsdContext, AnimationPayLoadData))
			{
				CurveAnimationPayloads[PayloadIndex].Emplace(AnimationPayLoadData);
			}
		};

		int32 CurvePayloadCount = CurveQueryIndexes.Num();
		CurveAnimationPayloads.AddDefaulted(CurvePayloadCount);
		const int32 BatchSize = 10;
		if (CurvePayloadCount > BatchSize)
		{
			const int32 NumBatches = (CurvePayloadCount / BatchSize) + 1;
			ParallelFor(
				NumBatches,
				[&CurveQueryIndexes, &GetAnimPayloadLambda](int32 BatchIndex)
				{
					int32 PayloadIndexOffset = BatchIndex * BatchSize;
					for (int32 PayloadIndex = PayloadIndexOffset; PayloadIndex < PayloadIndexOffset + BatchSize; ++PayloadIndex)
					{
						// The last batch can be incomplete
						if (!CurveQueryIndexes.IsValidIndex(PayloadIndex))
						{
							break;
						}
						GetAnimPayloadLambda(CurveQueryIndexes[PayloadIndex]);
					}
				},
				EParallelForFlags::BackgroundPriority
			);
		}
		else
		{
			for (int32 PayloadIndex = 0; PayloadIndex < CurvePayloadCount; ++PayloadIndex)
			{
				int32 PayloadQueriesIndex = CurveQueryIndexes[PayloadIndex];
				if (PayloadQueries.IsValidIndex(PayloadQueriesIndex))
				{
					GetAnimPayloadLambda(PayloadQueriesIndex);
				}
			}
		}

		for (TArray<UE::Interchange::FAnimationPayloadData>& AnimationPayload : CurveAnimationPayloads)
		{
			InOutPayloadData.Append(AnimationPayload);
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#undef LOCTEXT_NAMESPACE

#endif	  // USE_USD_SDK
