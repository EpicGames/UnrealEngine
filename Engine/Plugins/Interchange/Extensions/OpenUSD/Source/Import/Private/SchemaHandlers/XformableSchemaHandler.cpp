// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "SchemaHandlers/XformableSchemaHandler.h"

#include "InterchangeUsdContext.h"
#include "InterchangeUsdTranslator.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "USDPrimConversion.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdTyped.h"

#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"

namespace UE::Interchange::USD
{
	const FString& FXformableSchemaHandler::GetHandlerName() const
	{
		const static FString HandlerName = TEXT("XformableHandler");
		return HandlerName;
	}

	const FString& FXformableSchemaHandler::GetTargetSchemaName() const
	{
		const static FString SchemaName = TEXT("Xformable");
		return SchemaName;
	}

	bool FXformableSchemaHandler::OnTranslate(
		const UE::FUsdPrim& Prim,
		FTraversalInfo& TraversalInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FXformableSchemaHandler::OnTranslate)


		UInterchangeBaseNode* SceneNodeBase = AccumulatedInfo.GetOrCreateMainSceneNode(Prim, TraversalInfo, UsdContext);
		if (!SceneNodeBase)
		{
			return false;
		}

		// If we're an Xformable, get our transform.
		// All SceneNodes should have their LocalTransform set though.
		// Not setting will cause ensure hits in Skeleton generations for example.
		FTransform Transform = FTransform::Identity;
		bool bResetTransformStack = false;
		UsdToUnreal::ConvertXformable(Prim.GetStage(), UE::FUsdTyped(Prim), Transform, UsdUtils::GetEarliestTimeCode(), &bResetTransformStack);

		if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(SceneNodeBase))
		{
			SceneNode->SetCustomLocalTransform(UsdContext.GetNodeContainer(), Transform);
		}
		else if (UInterchangeSceneComponentNode* ComponentNode = Cast<UInterchangeSceneComponentNode>(SceneNodeBase))
		{
			ComponentNode->SetCustomLocalTransform(Transform);
		}

		// While we're adding the animation nodes here, we don't need to override and implement the payload data retrieval functions
		// on this handler as the ImageableSchemaHandler (which should always run when this runs as Imageable is a base class of
		// Xformable) already generically handles all animated property tracks
		if (UsdUtils::HasAnimatedTransform(Prim))
		{
			// TODO: This is likely not going to work with component nodes... For now only PointInstancers produce them however, and we've never
			// supported animating point instancers anyway so it may not be an issue yet
			AddTransformAnimationNode(Prim, AccumulatedInfo, UsdContext);
		}

		return true;
	}
}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
