// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Delegates/IDelegateInstance.h"

namespace UE::Interchange::USD
{
	using FSchemaHandlerIdentifier = int32;
}

class FInterchangeOpenUSDImportModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	int32 GetNumDefaultHandlers() const;

#if USE_USD_SDK
private:
	TArray<FString> RegisteredHandlers;
#endif	  // USE_USD_SDK
};
