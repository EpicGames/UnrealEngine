// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeUsdContext;
namespace UE
{
	class FSdfPath;
	class FUsdPrim;

	namespace UsdInfoCache::Private
	{
		struct FInterchangeUsdInfoCacheImpl;
	}
}

/**
 * Caches information about a specific USD Stage relevant for an Interchange import
 */
class FInterchangeUsdInfoCache
{
public:
	UE_API FInterchangeUsdInfoCache();
	UE_API virtual ~FInterchangeUsdInfoCache();

	// Returns whether we contain any info about prim at 'Path' at all
	UE_API bool ContainsInfoAboutPrim(const UE::FSdfPath& Path) const;

	UE_API void Build(UInterchangeUsdContext& Context);

	UE_API void Clear();
	UE_API bool IsEmpty();

public:
	UE_API bool IsPathCollapsed(const UE::FSdfPath& Path) const;
	UE_API bool DoesPathCollapseChildren(const UE::FSdfPath& Path) const;

	// Returns Path in case it represents an uncollapsed prim, or returns the path to the prim that collapsed it.
	// i.e. this always returns the root of the collapsed subtree that Path is involved in, if any.
	UE_API UE::FSdfPath UnwindToNonCollapsedPath(const UE::FSdfPath& Path) const;

private:
	TPimplPtr<UE::UsdInfoCache::Private::FInterchangeUsdInfoCacheImpl> Impl;
};

#undef UE_API

#endif // USE_USD_SDK