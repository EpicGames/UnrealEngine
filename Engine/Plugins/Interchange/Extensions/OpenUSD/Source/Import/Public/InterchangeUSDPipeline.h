// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Usd/InterchangeUsdDefinitions.h"

#include "InterchangeUSDPipeline.generated.h"

class UInterchangeBaseNodeContainer;

UCLASS(BlueprintType)
class INTERCHANGEOPENUSDIMPORT_API UInterchangeUsdPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

	UInterchangeUsdPipeline();

public:
	UPROPERTY(
		EditAnywhere,
		BlueprintReadWrite,
		Category = "Common",
		meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True")
	)
	FString PipelineDisplayName;

	/** Only import translated nodes from imageable prims (Xforms, Meshes, etc.) with these specific purposes from the USD file */
	UE_DEPRECATED(5.8, "This option has moved back to the USD translator settings")
	UPROPERTY(meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdPurpose"))
	int32 GeometryPurpose;

	/** Setting to tell what primvars have to be attached to the MeshDescription */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	EInterchangeUsdPrimvar ImportPrimvars = EInterchangeUsdPrimvar::Bake;

	/**
	 * Subdivision level to use for all subdivision meshes on the opened stage. 0 means "don't subdivide".
	 * The maximum level of subdivision allowed can be configured via the 'USD.Subdiv.MaxSubdivLevel' cvar.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD", meta = (ClampMin = "0", ClampMax = "6", UIMin = "0", UIMax = "6"))
	int32 SubdivisionLevel;

	/**
	 * The translator always emits a scene node for the stage pseudoroot named after the stage's root layer filename.
	 * Setting this option to false means we will remove it from the factory nodes, making top-level prims lead to top-level actors and components.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	bool bImportPseudoRoot;

	/**
	 * Imported meshes will place their primvars on arbitrary UV set indices (e.g. "st2" on UV0, "st3" on UV1, etc.). Imported materials may be
	 * expecting to read from specific primvars on particular UV indices (e.g. "st1" from UV0, "st2" from UV2, etc.). These mappings won't
	 * necessarily correspond to each other. Setting this option to true means we'll generate additional material instances with primvar-UV index
	 * mappings made to best match the exact meshes they are assigned to (e.g. "st1" disabled, "st2" from UV0). Setting this to false means we
	 * will just assign the incompatible materials to the meshes anyway.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	bool bGeneratePrimvarCompatibleMaterials;

public:

	virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;

protected:
	virtual void ExecutePipeline(
		UInterchangeBaseNodeContainer* BaseNodeContainer,
		const TArray<UInterchangeSourceData*>& SourceDatas,
		const FString& ContentBasePath
	) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override;
};
