// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeUSDInfoCache.h"
#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/HandlerAccumulatedInfo.h"
#include "USDGeomMeshConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"

#include "USDMaterialXShaderGraph.h"

#include "CoreMinimal.h"

#include "InterchangeUsdContext.generated.h"

class FUsdInfoCache;
class UInterchangeAnimationTrackSetNode;
class UInterchangeBaseNodeContainer;
class UInterchangeResultsContainer;
class UInterchangeTranslatorBase;
class UInterchangeUSDTranslator;
class UInterchangeUsdTranslatorSettings;
namespace UE::Interchange::USD
{
	class FSchemaHandler;
}

UCLASS(BlueprintType)
class INTERCHANGEOPENUSDIMPORT_API UInterchangeUsdContext : public UObject
{
	GENERATED_BODY()

public:
	UInterchangeUsdContext();
	virtual void BeginDestroy() override;

	void Initialize(UInterchangeUSDTranslator* Translator, UInterchangeBaseNodeContainer* NodeContainer);

	/**
	 * Generates a UID for an asset node (mesh, material, texture, light, etc.) associated with a prim.
	 * Schema handlers should call this instead of directly building UIDs via MakeNodeUid().
	 *
	 * By default produces MakeNodeUid(Prefix + GetPrototypePrimPath(Prim) + Suffix).
	 * Subclasses can override to produce remapped UIDs (e.g. for pregen target-based deduplication).
	 */
	virtual FString MakeAssetNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix = FString(), const FString& Suffix = FString()) const;

	/**
	 * Generates a UID for a scene node (actor/component) associated with a prim.
	 * Schema handlers should call this instead of directly building UIDs via MakeNodeUid().
	 *
	 * By default produces MakeNodeUid(Prim.GetPrimPath() + Suffix).
	 * Scene nodes are per-instance (not deduplicated), so this uses the prim's own path.
	 */
	virtual FString MakeSceneNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix = FString(), const FString& Suffix = FString()) const;

	// Returns the ID the provided stage has within the USDUtils' singleton StageCache.
	UFUNCTION(BlueprintCallable, Category = "Interchange | USD")
	int64 GetStageId() const;

	// Sets the ID of a particular stage from the UsdUtils' singleton StageCache.
	// If this corresponds to a valid USD Stage, that stage will be used for the Interchange import.
	UFUNCTION(BlueprintCallable, Category = "Interchange | USD")
	void SetStageId(int64 InStageId);

	// Convenience functions to get/set the stage directly, although it will internally just
	// get/set the stage into the UsdUtils' singleton StageCache and track it's Id instead.
	UE::FUsdStage GetUsdStage() const;
	bool SetUsdStage(const UE::FUsdStage& InStage);

	UE_DEPRECATED(5.8, "Use the SetupInterchangeInfoCache() and GetInterchangeInfoCache() functions instead")
	FUsdInfoCache* GetInfoCache() const;

	UE_DEPRECATED(5.8, "Use the SetupInterchangeInfoCache() and GetInterchangeInfoCache() functions instead")
	void SetExternalInfoCache(FUsdInfoCache& InInfoCache);

	UE_DEPRECATED(5.8, "Use the SetupInterchangeInfoCache() and GetInterchangeInfoCache() functions instead")
	FUsdInfoCache* CreateOwnedInfoCache();

	UE_DEPRECATED(5.8, "Use the SetupInterchangeInfoCache() and GetInterchangeInfoCache() functions instead")
	void BuildInfoCache();

	UE_DEPRECATED(5.8, "Use the SetupInterchangeInfoCache() and GetInterchangeInfoCache() functions instead")
	void ReleaseInfoCache();

	void Reset();

	void SetupTrackSetNode();

	void SetupInterchangeInfoCache();

	UInterchangeUSDTranslator* GetTranslator() const;
	UInterchangeUsdTranslatorSettings* GetTranslatorSettings() const;
	UInterchangeResultsContainer* GetResultsContainer() const;
	UInterchangeBaseNodeContainer* GetNodeContainer() const;
#if USE_USD_SDK
	FInterchangeUsdInfoCache* GetInterchangeInfoCache() const;
#endif	  // USE_USD_SDK

private:
	void SetupMeshConversionOptions();

public:
#if USE_USD_SDK
	// Unique identifier to other translators that we spawn in the process of translating the file and producing payloads,
	// in order to handle other format types (MaterialX, OpenVDB, etc.)
	TMap<FString, TStrongObjectPtr<UInterchangeTranslatorBase>> SubTranslators;

	// This node eventually becomes a LevelSequence, and all track nodes are connected to it.
	// For now we only generate a single LevelSequence per stage though, so we'll keep track of this
	// here for easy access when parsing tracks
	TObjectPtr<UInterchangeAnimationTrackSetNode> CurrentTrackSet = nullptr;

	// When traversing we'll generate FTraversalInfo objects. If we need to (e.g. for skinned meshes),
	// we'll store the info for that translated node here, so we don't have to recompute it when returning
	// the payload data.
	// Note: We only do this when needed: This shouldn't have data for every prim in the stage.
	TMap<FString, UE::Interchange::USD::FTraversalInfo> NodeUidToCachedTraversalInfo;
	mutable FRWLock NodeUidToCachedTraversalInfoLock;

	// We store temp stages in here that we open in order to parse stuff inside of inactive variants of our main UsdStage.
	// We do this because the payload data are retrieved concurrently, and toggling variants mutates the current stage.
	mutable FRWLock PrimPathToVariantToStageLock;
	TMap<FString, TMap<FString, UE::FUsdStage>> PrimPathToVariantToStage;

	// On UInterchangeUSDTranslator::Translate we set this up based on our TranslatorSettings, and then
	// we can reuse it (otherwise we have to keep converting the FNames into Tokens all the time)
	UsdToUnreal::FUsdMeshConversionOptions CachedMeshConversionOptions;

	TMap<FString, UsdUtils::FUsdPrimMaterialAssignmentInfo> CachedMaterialAssignments;

	FString USDZFilePath;
	FString DecompressedUSDZRoot;

	// Additional information extracted from MaterialX materials.
	// We stash these here because we need both materials and meshes to process the geomprop into primvars properly, but
	// we may run into the Materials first during parsing.
	// TODO: Maybe remove and use info cache?
	TMap<FString, TArray<FUsdMaterialXShaderGraph::FGeomProp>> MaterialUidToGeomProps;

	// The regular UsdTranslator traversal will not translate prims whose paths are in this map.
	// The UsdTranslator will also automatically mark prims as visited during its regular traversal.
	//
	// This can be use to prevent the regular traversal from visiting prims that a schema handler for a parent prim may have
	// already processed
	TMap<UE::FSdfPath, UE::Interchange::USD::FHandlerAccumulatedInfo> HandledPrimInfo;

	// Parsed view of UInterchangeUsdTranslatorSettings::PrimsToImport, populated at the top of
	// UInterchangeUSDTranslator::Translate(). The translator's TranslatePrimSubtree consults
	// this to skip out-of-scope subtrees, and schema handlers driving traversal directly may
	// want to honor it as well. Empty means "no filter" (the fast path); the full-stage default
	// of {"/"} also resolves to empty here.
	TArray<UE::FSdfPath> ParsedPrimsToImport;

private:
	TObjectPtr<UInterchangeUSDTranslator> Translator;
	TObjectPtr<UInterchangeBaseNodeContainer> NodeContainer;

	// We never store the stage itself, but only it's Id within the UsdUtils' singleton StageCache.
	// The intent here is to allow Python stages to be passed in and manipulated via Python in case of Python imports or pipelines.
	int64 StageIdInUsdUtilsStageCache = INDEX_NONE;
	bool bShouldCleanUpFromStageCache = false;

	TUniquePtr<FInterchangeUsdInfoCache> InfoCache;
#endif	  // USE_USD_SDK
};
