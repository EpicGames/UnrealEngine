// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "Audio/InterchangeAudioPayloadInterface.h"
#include "Groom/InterchangeGroomPayloadInterface.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeGenericPayloadInterface.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Texture/InterchangeBlockedTexturePayloadInterface.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "Volume/InterchangeVolumePayloadInterface.h"

#include "InterchangeUsdTraversalInfo.h"
#include "SchemaHandlers/SchemaHandlerEntry.h"
#include "USDStageOptions.h"

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeUsdTranslator.generated.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeSceneNode;
class UInterchangeGenericPayloadData;
class UInterchangeUsdContext;
enum class EUsdInterpolationType : uint8;
struct FInterchangeGroomPayloadKey;
namespace UE::InterchangeUsdTranslator::Private
{
	class UInterchangeUSDTranslatorImpl;
}
namespace UE
{
	class FUsdPrim;
	namespace Interchange
	{
		struct FGroomPayloadData;

		namespace USD
		{
			class FHandlerAccumulatedInfo;
			class FSchemaHandler;
		}
	}
}

/** Describes what to do when we're collapsing and encounter a PointInstancer prim in a prim subtree */
UENUM(BlueprintType)
enum class EUsdPointInstancerCollapsing : uint8
{
	/** Prevent collapsing the subtree the PointInstancer occupies */
	NoCollapsing,

	/** Collapse the point instancer along with the rest of the subtree, baking its instances into the produced StaticMesh */
	CollapseAsStaticMesh
};

UCLASS(BlueprintType, editinlinenew, MinimalAPI)
class UInterchangeUsdTranslatorSettings : public UInterchangeTranslatorSettings
{
	GENERATED_BODY()

public:
	/** Only import geometry prims with these specific purposes from the USD file */
	UPROPERTY(EditAnywhere, Category = "USD Translator", meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdPurpose"))
	int32 GeometryPurpose;

	/** Specifies which set of shaders to use when parsing USD materials, in addition to the universal render context. */
	UE_DEPRECATED(5.6, "This is no longer used: Reorder and enable/disable the registered material schema handlers to control what render contexts are parsed (see the CustomHandlerEntries property).")
	UPROPERTY()
	FName RenderContext;

	/** Specifies which material purpose to use when parsing USD material bindings, in addition to the "allPurpose" fallback */
	UPROPERTY(EditAnywhere, Category = "USD Translator")
	FName MaterialPurpose;

	/** Describes how to interpolate between a timeSample value and the next */
	UPROPERTY(EditAnywhere, Category = "USD Translator")
	EUsdInterpolationType InterpolationType;

	/** Whether to use the specified StageOptions instead of the stage's own settings */
	UPROPERTY(EditAnywhere, Category = "USD Translator")
	bool bOverrideStageOptions;

	/** Custom StageOptions to use for the stage */
	UPROPERTY(EditAnywhere, Category = "USD Translator", meta = (EditCondition = bOverrideStageOptions))
	FUsdStageOptions StageOptions;

	/** How the USD Translator should behave when it tries collapsing a subtree and encounters a PointInstancer prim */
	UPROPERTY(EditAnywhere, Category = "USD Translator|Collapsing")
	EUsdPointInstancerCollapsing PointInstancerCollapsing;

	/**
	 * Whether to use the dedicated collapsing schema to collapse prim subtrees into single meshes
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator|Collapsing")
	bool bUseSchemaForCollapsing;

	/**
	 * Use KindsToCollapse to determine when to collapse prim subtrees or not (defaults to disabled on Interchange).
	 * Disable this if you want to prevent collapsing, or to control it manually by right-clicking on individual prims.
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator|Collapsing")
	bool bUsePrimKindsForCollapsing;

	/**
	 * Which prim kinds that are allowed to collapse its children and/or be collapsed.
	 * See also the "USD.CollapsePrimsWithoutKind" cvar.
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator|Collapsing", meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdDefaultKind", EditCondition = bUsePrimKindsForCollapsing))
	int32 KindsToCollapse;

	/**
	 * Whether to convert USD prim default attribute opinions (i.e. non-animated) into translated node user attributes, ultimately ending up imported as asset and actor metadata.
	 *
	 * Note that AttributeRegexFilter may be needed in order to select only the desired metadata, as otherwise regular attributes like 
	 * 'points' and 'faceVertexCounts' will also get translated into Interchange metadata, which could be expensive.
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator")
	bool bTranslatePrimAttributes;

	/** Regex to match against USD prim attribute names to decide whether to convert them into translated node user attributes */
	UPROPERTY(EditAnywhere, Category = "USD Translator", meta = (EditCondition = bTranslatePrimAttributes))
	FString AttributeRegexFilter;

	/** Whether to convert USD prim metadata into translated node user attributes, if they pass the regex filter. See MetadataRegexFilter. */
	UPROPERTY(EditAnywhere, Category = "USD Translator")
	bool bTranslatePrimMetadata;

	/**
	 * Regex to match against the full USD prim metadata key to decide whether to convert them into translated node user attributes.
	 *
	 * Note that nested dictionaries are flattened using the ":" character as separator. So a myMetadata key inside the customData
	 * dictionary will end up with the full key 'customData:myMetadata', and could be selected by a regex filter 'customData:my',
	 * or just 'custom', or 'my', etc.
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator", meta = (EditCondition = bTranslatePrimMetadata))
	FString MetadataRegexFilter;

	/**
	 * List of prim paths to import. Defaults to just ["/"], which imports the entire stage.
	 *
	 * Example: [TEXT("/Root/MySubtree"), TEXT("/Classes/MyClassPrim")]
	 *
	 * Note that unlike the other settings this does not persist across imports, as it is specific to each individual stage.
	 */
	UPROPERTY(EditAnywhere, Transient, Category = "USD Translator", meta = (DisplayName = "Prims to Import"))
	TArray<FString> PrimsToImport = TArray<FString>{TEXT("/")};

	/**
	 * Custom array of schema handlers to use for translation.
	 * When this is empty, the default FSchemaHandlerRegistry::RegisteredHandlerEntries array will be used instead.
	 *
	 * It is expected for this array to essentially be a reorder of FSchemaHandlerRegistry::RegisteredHandlerEntries, with additionally
	 * some modified property values on schema handler entries, if needed (e.g. enabled or custom render contexts).
	 *
	 * It is not sufficient to create a new FSchemaHandlerEntry instance from scratch and insert here directly: If you have a custom handler,
	 * you want to register it on FSchemaHandlerRegistry instead, so as to also get the handler generator function correctly registered.
	 * At that point, your handler will also be added to FSchemaHandlerRegistry::RegisteredHandlerEntries.
	 */
	UPROPERTY(EditAnywhere, Category = "USD Translator|Handlers", AdvancedDisplay)
	TArray<FSchemaHandlerEntry> CustomHandlerEntries;

public:
	UInterchangeUsdTranslatorSettings();
};

UCLASS(BlueprintType)
class UInterchangeUSDTranslator
	: public UInterchangeTranslatorBase
	, public IInterchangeMeshPayloadInterface
	, public IInterchangeTexturePayloadInterface
	, public IInterchangeBlockedTexturePayloadInterface
	, public IInterchangeAnimationPayloadInterface
	, public IInterchangeVolumePayloadInterface
	, public IInterchangeGroomPayloadInterface
	, public IInterchangeAudioPayloadInterface
	, public IInterchangeGenericPayloadInterface
{
	GENERATED_BODY()

public:
	UInterchangeUSDTranslator();

	/** Begin UInterchangeTranslatorBase API*/
	virtual EInterchangeTranslatorType GetTranslatorType() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;
	virtual TArray<FString> GetSupportedFormats() const override;
	virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;
	virtual void ReleaseSource() override;
	virtual void ImportFinish() override;
	virtual UInterchangeTranslatorSettings* GetSettings() const override;
	virtual void SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings) override;
	/** End UInterchangeTranslatorBase API*/

#if USE_USD_SDK
	/**
	 * Invokes all registered handlers on Prim, producing Interchange translated nodes and adding them to our UsdContext's NodeContainer.
	 * The accumulated products from all those handlers is returned as FHandlerAccumulatedInfo, if translation succeeded.
	 *
	 * This will also mark the Prim as visited on the UsdContext's HandledPrimInfo member, caching the produced FHandlerAccumulatedInfo.
	 * Repeated calls for the same prim will just return that cached info instead.
	 */
	UE_API virtual TOptional<UE::Interchange::USD::FHandlerAccumulatedInfo> TranslatePrim(
		const UE::FUsdPrim& Prim,
		UE::Interchange::USD::FTraversalInfo& TraversalInfo,
		bool bAllowSceneNodeGeneration = true
	) const;

	/**
	 * Recursively traverses their Prim and its children, invoking TranslatePrim().
	 *
	 * Traversal will not step into prims if they are already on the UsdContext's HandledPrimInfo member.
	 *
	 * In general, the regular Translate() implementation will traverse the full stage, recursively calling this function. This is exposed
	 * here however in case your schema handler needs to take charge of the stage traversal for whatever reason. You can then choose to
	 * process the prims you wish, and then yield back control of the rest of the subtree traversal to the UInterchangeUSDTranslator by 
	 * calling this function for that subtree.
	 */
	UE_API virtual void TranslatePrimSubtree(const UE::FUsdPrim& Prim, const UE::Interchange::USD::FTraversalInfo& TraversalInfo, bool bAllowSceneNodeGeneration = true, UE::Interchange::USD::FHandlerAccumulatedInfo* SubTreeAccumulatedInfo = nullptr) const;

	/**
	 * Returns the schema handlers that are currently being used by the translator.
	 * These are created during the Translate() call, and released when the import is complete.
	 */
	UE_API virtual const TArray<TSharedRef<UE::Interchange::USD::FSchemaHandler>>& GetCurrentSchemaHandlers() const;
#endif	  // USE_USD_SDK

	/** Begin Interchange payload interfaces */
    UE_DEPRECATED(5.6, "Deprecated. Use GetMeshPayloadData(const FInterchangeMeshPayLoadKey&) instead.")
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(
		const FInterchangeMeshPayLoadKey& PayLoadKey,
		const FTransform& MeshGlobalTransform
	) const override
    {
		using namespace UE::Interchange;
		UE::Interchange::FAttributeStorage Attributes;
		Attributes.RegisterAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);
        return GetMeshPayloadData(PayLoadKey, Attributes);
    }

    virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const override;
	virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	virtual TOptional<UE::Interchange::FImportBlockedImage> GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	virtual TArray<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const override;
	virtual TOptional<UE::Interchange::FVolumePayloadData> GetVolumePayloadData(const UE::Interchange::FVolumePayloadKey& PayloadKey) const override;
	virtual TOptional<UE::Interchange::FGroomPayloadData> GetGroomPayloadData(const FInterchangeGroomPayloadKey& PayloadKey) const override;
	virtual TOptional<UE::Interchange::FInterchangeAudioPayloadData> GetAudioPayloadData(const FString& PayloadSourceFileKey) const override;
	virtual TObjectPtr<UInterchangeGenericPayloadData> GetGenericPayloadData(const FString& PayloadKey) const override;
	/** End Interchange payload interfaces */

private:
	mutable TUniquePtr<UE::InterchangeUsdTranslator::Private::UInterchangeUSDTranslatorImpl> Impl;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UInterchangeUsdTranslatorSettings> TranslatorSettings = nullptr;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UInterchangeUsdContext> UsdContext = nullptr;
};

#undef UE_API