// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "Templates/SharedPointer.h"

#include "UsdWrappers/ForwardDeclarations.h"

class UInterchangeBaseNode;
class UInterchangeMeshBundleNode;
namespace UE
{
	class FUsdSkelCache;
	class FUsdPrim;
	class FUsdSkelSkeletonQuery;
}
namespace UsdToUnreal
{
	namespace NaniteAssemblyUtils
	{
		class FNaniteAssemblyTraversalResult;
	}
}

namespace UE::Interchange::USD
{
	// Information intended to be passed down from parent to children (by value) as we traverse the stage
	struct FTraversalInfo
	{
		using FNaniteAssemblyTraversalResult = UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult;

		UInterchangeBaseNode* ParentNode = nullptr;

		FTransform SceneGlobalTransform;

		// Whether we're already collapsing a subtree or not (used to prevent us from creating nested mesh bundles)
		bool bCurrentlyCollapsing = false;

		TSharedPtr<UE::FUsdSkelCache> FurthestSkelCache;
		TSharedPtr<FString> FurthestParentSkelRootPath;	   // Used to populate the skel cache
		TSharedPtr<FString> ClosestParentSkelRootPath;
		TSharedPtr<FString> BoundSkeletonPrimPath;
		TSharedPtr<TArray<FString>> SkelJointUsdNames;	  // Needed for skel mesh payloads

		// Nanite Assembly info
		TSharedPtr<FString> ClosestNaniteAssemblySkelBindingPath;
		TSharedPtr<const FNaniteAssemblyTraversalResult> NaniteAssemblyTraversalResult;

		bool bVisible = true;
		bool bInsideLODVariant = false;
		bool bIsLODVariantContainer = false;

	public:
		void UpdateWithCurrentPrim(const UE::FUsdPrim& CurrentPrim);
		UE::FUsdSkelSkeletonQuery ResolveSkelQuery(const UE::FUsdStage& Stage) const;
		UE::FUsdPrim ResolveClosestParentSkelRoot(const UE::FUsdStage& Stage) const;
		void RepopulateSkelCache(const UE::FUsdStage& Stage);
	};

}	 // namespace UE::Interchange::USD

#endif	  // USE_USD_SDK
