// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandler.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

namespace UE::Interchange::USD
{
	class FGroomSchemaHandler : public FSchemaHandler
	{
	public:
		UE_API virtual const FString& GetHandlerName() const override;

		UE_API virtual const FString& GetTargetSchemaName() const override;

		UE_API virtual bool CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual bool OnTranslate(
			const UE::FUsdPrim& Prim,
			FTraversalInfo& TraversalInfo,
			FHandlerAccumulatedInfo& AccumulatedInfo,
			UInterchangeUsdContext& UsdContext
		) override;

		UE_API virtual bool OnGetGroomPayloadData(
			const FInterchangeGroomPayloadKey& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FGroomPayloadData>& InOutPayloadData
		);
	};
}

#undef UE_API

#endif	  // USE_USD_SDK