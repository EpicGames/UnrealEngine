// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "InterchangeMeshNode.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Texture/InterchangeBlockedTexturePayloadData.h"
#include "Texture/InterchangeTexturePayloadData.h"

#include "UsdWrappers/UsdPrim.h"

#include "Containers/AnsiString.h"
#include "Misc/Optional.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeBaseNode;
class UInterchangeUsdContext;
namespace UE
{
	namespace Interchange
	{
		namespace USD
		{
			struct FTraversalInfo;
		}
	}
}

namespace UE::Interchange::USD
{
	/**
	 * When we translate a USD prim via the USD translator, we will invoke every suitable schema handler in order.
	 * This class is used to accumulate the data produced by each schema handler, and is passed as input/output argument
	 * to each handler invocation, so that they can share/accumulate the information they produce.
	 *
	 * For example, before creating a new Mesh node for a prim, you should check whether one was already created and
	 * added to PrimAssetNodes. If so, you should probably try reusing that node instead of creating another one
	 */
	class FHandlerAccumulatedInfo
	{
	public:
		// First node is taken as the main scene node, and passed as ParentNode on FTraversalInfo. Note that this main scene node
		// may be an actually "nullptr" value. This would mean that there should not be any scene node translated for this particular
		// prim (e.g. due to UInterchangeUSDTranslator::Translate() having been called with bAllowSceneNodeGeneration)
		//
		// We don't store actual UInterchangeSceneNode* here because some "scene" nodes don't derive from it directly (e.g. UInterchangeSceneComponentNode)
		TArray<UInterchangeBaseNode*> PrimSceneNodes;
		TArray<UInterchangeBaseNode*> PrimAssetNodes;

	public:
		/** 
		 * The "main scene node" is the first node in PrimSceneNodes, always (even if nullptr, which allows the caller to
		 * control whether scene node information is produced or not. See also the comment on the PrimSceneNodes member).
		 *
		 * If your handler intends to produce scene node information, it should retrieve it via GetOrCreateMainSceneNode().
		 */
		UE_API UInterchangeBaseNode* GetMainSceneNode();
		UE_API UInterchangeBaseNode* GetOrCreateMainSceneNode(
			const UE::FUsdPrim& Prim,
			const FTraversalInfo& TraversalInfo,
			UInterchangeUsdContext& UsdContext
		);

		/**
		 * Returns the first node found in PrimAssetNodes that matches the desired class.
		 * If none of that class can be found, a new one will be created, added to PrimAssetNodes, and returned.
		 */
		UE_API UInterchangeBaseNode* GetOrCreateAssetNode(
			const TSubclassOf<UInterchangeBaseNode>& NodeClass,
			UInterchangeBaseNodeContainer& NodeContainer,
			const FString& NewNodeUid,
			const FString& NewNodeDisplayName,
			bool* bOutCreatedNode = nullptr
		);
		template<typename T>
		T* GetOrCreateAssetNode(
			UInterchangeBaseNodeContainer& NodeContainer,
			const FString& NewNodeUid,
			const FString& NewNodeDisplayName,
			bool* bOutCreatedNode = nullptr
		)
		{
			return Cast<T>(GetOrCreateAssetNode(T::StaticClass(), NodeContainer, NewNodeUid, NewNodeDisplayName, bOutCreatedNode));
		}

		/**
		 * Returns the first node found in PrimAssetNodes that matches the desired class.
		 * If none of that class can be found, returns nullptr.
		 */
		UE_API UInterchangeBaseNode* GetAssetNodeOfClass(const TSubclassOf<UInterchangeBaseNode>& NodeClass);
		template<typename T>
		T* GetAssetNodeOfClass()
		{
			return Cast<T>(GetAssetNodeOfClass(T::StaticClass()));
		}

		/** Appends the data from 'Other' into this accumulated info */
		UE_API void AppendInfo(const FHandlerAccumulatedInfo& Other);
		UE_API void AppendInfo(FHandlerAccumulatedInfo&& Other);
	};
}

#undef UE_API

#endif	  // USE_USD_SDK