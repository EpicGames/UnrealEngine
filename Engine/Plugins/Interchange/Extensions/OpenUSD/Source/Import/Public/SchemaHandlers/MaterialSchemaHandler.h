// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandler.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

namespace UE::Interchange::USD
{
	class FMaterialSchemaHandler : public FSchemaHandler
	{
	public:
		UE_API virtual const FString& GetTargetSchemaName() const override;

		/**
		 * Returns the list of render contexts that this handler supports by default.
		 * This is likely to be a single value, such as 'unreal' or 'mtlx'
		 */
		UE_API virtual const TArray<FString>& GetDefaultRenderContexts() const = 0;

		/**
		 * Returns whether this handler allows the user to provide custom render contexts.
		 * This being false (the default value) means this handler is hard-coded to work with only specific render contexts.
		 */
		UE_API virtual bool AllowCustomRenderContexts() const;

		/**
		 * Returns the full list of render contexts that this handler will try parsing materials with.
		 * This value is initialized with the output of GetDefaultRenderContexts(), so it should include those entries by default.
		 */
		UE_API virtual const TArray<FString>& GetCustomRenderContexts() const;

		/**
		 * Sets the full list of render contexts that this handler will try parsing materials with, if AllowCustomRenderContexts()
		 * is enabled.
		 */
		UE_API virtual void SetCustomRenderContexts(const TArray<FString>& RenderContexts);

		UE_API virtual bool CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const override;

	protected:
		TArray<FString> CustomRenderContexts;
	};
}

#undef UE_API

#endif	  // USE_USD_SDK