// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "InterchangeMeshNode.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Texture/InterchangeBlockedTexturePayloadData.h"
#include "Texture/InterchangeTexturePayloadData.h"

#include "SchemaHandlers/SchemaHandlerEntry.h"
#include "UsdWrappers/UsdPrim.h"

#include "Containers/AnsiString.h"
#include "Misc/Optional.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeBaseNode;
class UInterchangeSceneNode;
class UInterchangeGenericPayloadData;
class UInterchangeUsdContext;
struct FInterchangeGroomPayloadKey;
struct FInterchangeMeshPayLoadKey;
namespace UE
{
	class FUsdPrim;
	namespace Interchange
	{
		class FAttributeStorage;
		struct FAnimationPayloadData;
		struct FAnimationPayloadQuery;
		struct FGroomPayloadData;
		struct FImportBlockedImage;
		struct FImportImage;
		struct FInterchangeAudioPayloadData;
		struct FMeshPayloadData;
		struct FVolumePayloadData;
		struct FVolumePayloadKey;

		namespace USD
		{
			class FHandlerAccumulatedInfo;
			class FSchemaHandler;
			struct FTraversalInfo;
		}
	}
}

namespace UE::Interchange::USD
{
	/**
	 * Class that can be used to encapsulate the translation of prims of a certain type into Interchange nodes.
	 *
	 * You can create a custom handler to process a prim with a target schema by deriving this class and overriding its functions. You must
	 * then register the class with FSchemaHandlerRegistry::Register(). Whenever a new USD file is imported, a new UInterchangeUSDTranslator
	 * is created, spawning instances of all registered handlers. The translator will retain those instances for the entire translation process.
	 *
	 * Within the UInterchangeUSDTranslator's TranslatePrim() call for a given prim, the translator will go through all of its spawned
	 * handlers in order, and for any handler that returns true to CanHandlePrim() it will invoke OnTranslate(). Note that many handlers
	 * can be invoked for each individual prim, so that in the general case handlers should try to reuse and append information to the
	 * accumulated Interchange nodes created for that prim.
	 *
	 * Whenever you don't want your handler to be invoked anymore (e.g. when your module is being shut down) you can unregister
	 * your handler with FSchemaHandlerRegistry::Unregister(), passing the handler name
	 */
	class FSchemaHandler
	{
	public:
		UE_API virtual ~FSchemaHandler();

		/** Returns the name of this handler, which must be unique between all registered handlers */
		virtual const FString& GetHandlerName() const = 0;

		/** Gets whether this handler is enabled for this translation or not */
		UE_API virtual bool IsEnabled() const;

		/** Sets whether this handler is enabled for this translation or not */
		UE_API virtual void SetEnabled(bool bEnabled);

		/**
		 * Returns the schema name of the prim type to which this handler applies ("Mesh", or "Material, or "Xform", etc.).
		 *
		 * The base implementation of CanHandlePrim() will use this to determine whether a handler applies to a prim 
		 * (i.e. it will call Prim.IsA(GetTargetSchemaName()))
		 */
		UE_API virtual const FString& GetTargetSchemaName() const;

		/**
		 * The InterchangeUSDTranslator only calls OnTranslate() for schema handlers that return true from this function.
		 * Additionally, the FInterchangeUsdInfoCache only ever queries CanBeCollapsed() and CollapsesChildren() for handlers that return true from
		 * this function
		 */
		UE_API virtual bool CanHandlePrim(const UE::FUsdPrim& Prim, const UInterchangeUsdContext& UsdContext) const;

		/**
		 * Returns whether this prim can be collapsed according to this schema handler.
		 * Returning an empty optional (the default) means this schema handler has no opinion about this, and yields to weaker schema handlers.
		 * This doesn't get queried during regular translation, but instead is used to build the FInterchangeUsdInfoCache.
		 */
		UE_API virtual TOptional<bool> CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const;

		/**
		 * Returns whether this prim should collapse its entire subtree into one asset / actor, according to this schema handler.
		 * Returning an empty optional (the default) means this schema handler has no opinion about this, and yields to weaker schema handlers.
		 * This doesn't get queried during regular translation, but instead is used to build the FInterchangeUsdInfoCache.
		 */
		UE_API virtual TOptional<bool> CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const;

		/**
		 * Main function that should be implemented in order to translate a prim into Interchange nodes.
		 * @param Prim - Prim to translate
		 * @param TraversalInfo - Stores hierarchical information that is collected while we traverse the stage from root to leaf prims
		 * @param AccumulatedInfo - Stores information generated by all schema handlers when invoked for this same Prim. If your handler creates
		 *                          Interchange nodes when processing a prim, they should be added here (in addition to the UsdContext's NodeContainer).
		 *                          Your handler should also first search for reusable nodes in this struct before creating new nodes, if possible
		 *                          (i.e. prioritize appending information)
		 * @param UsdContext - Global context for this USD stage translation, holding the stage, the node container, cached nodes, etc.
		 * @return Whether this handler has successfully processed the prim
		 */
		 UE_API virtual bool OnTranslate(
			 const UE::FUsdPrim& Prim,
			 FTraversalInfo& TraversalInfo,
			 FHandlerAccumulatedInfo& AccumulatedInfo,
			 UInterchangeUsdContext& UsdContext
			);

		/**
		 * Functions that can be implemented by your handler to provide payload data for a given payload key.
		 *
		 * When retrieving payload data for a given prim, the UInterchangeUSDTranslator will iterate through all of its spawned handlers in order
		 * and invoke this function, passing the same InOutPayloadData argument for all handlers. The intent here is to allow handlers to post-process
		 * the payload data if they desire.
		 *
		 * Note that unlike for OnTranslate(), CanHandlePrim() is not checked before invoking this function, and no other check is performed either.
		 * This means that your handler should itself validate within this function whether it should affect the payload data or not.
		 *
		 * See the corresponding Interchange interface classes for details on each individual payload data and parameter types.
		 *
		 * @param PayLoadKey - Key for the requested payload data
		 * @param PayloadAttributes - Attributes provided by the factories in addition to the payload key
		 * @param UsdContext - Global context for this USD stage translation, holding the stage, the node container, cached nodes, etc.
		 * @param InOutPayloadData - Accumulated payload information passed through all handlers for a particular payload request
		 * @return Whether this handler has successfully processed this payload data.
		 */
		UE_API virtual bool OnGetMeshPayloadData(
			const FInterchangeMeshPayLoadKey& PayLoadKey,
			const UE::Interchange::FAttributeStorage& PayloadAttributes,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FMeshPayloadData>& InOutPayloadData
		);
		UE_API virtual bool OnGetTexturePayloadData(
			const FString& PayloadKey,
			TOptional<FString>& AlternateTexturePath,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FImportImage>& InOutPayloadData
		);
		UE_API virtual bool OnGetBlockedTexturePayloadData(
			const FString& PayloadKey,
			TOptional<FString>& AlternateTexturePath,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FImportBlockedImage>& InOutPayloadData
		);
		UE_API virtual bool OnGetAnimationPayloadData(
			const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
			UInterchangeUsdContext& UsdContext,
			TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
		);
		UE_API virtual bool OnGetVolumePayloadData(
			const UE::Interchange::FVolumePayloadKey& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FVolumePayloadData>& InOutPayloadData
		);
		UE_API virtual bool OnGetGroomPayloadData(
			const FInterchangeGroomPayloadKey& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FGroomPayloadData>& InOutPayloadData
		);
		UE_API virtual bool OnGetAudioPayloadData(
			const FString& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FInterchangeAudioPayloadData>& InOutPayloadData
		);
		UE_API virtual bool OnGetGenericPayloadData(
			const FString& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TObjectPtr<UInterchangeGenericPayloadData>& InOutPayloadData
		);

	protected:
		bool bEnabled = true;
	};
}	 // namespace UE::Interchange::USD

#undef UE_API

#endif	  // USE_USD_SDK
