// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaHandlerEntry.generated.h"

/**
 * Represents a registered schema handler of a particular type.
 *
 * This class is used instead of directly tracking FSchemaHandlerGenerators in order to allow programmatically inspecting
 * and manipulating the registered persistent handler information without repeatedly instantiating handlers.
 *
 * It's a USTRUCT and kept in a separate file as it's used by UInterchangeUsdTranslatorSettings in order to track
 * custom schema handler settings (like ordering, which is enabled, etc.)
 */
USTRUCT(BlueprintType)
struct FSchemaHandlerEntry
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD Translator")
	FString HandlerName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Translator")
	bool bEnabled = true;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD Translator")
	FString SchemaName;

	/** For material schema handlers, we also hoist the render contexts the handler supports here, to be visible on the UI */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD Translator")
	TArray<FString> DefaultRenderContexts;

	/**
	 * If true, it means the handler allows the user to manually specify which render contexts to parse (e.g. the default 'universal' handler does this).
	 * If false, it means the handler is hard-coded to only look for a particular render context (e.g. the default MaterialX handler does this).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Translator")
	bool bAllowCustomRenderContexts = false;

	/**
	 * The full list of custom render contexts that this handler should try parsing.
	 * This is only used whenever bAllowCustomRenderContexts is enabled.
	 * This value is initialized with the value of DefaultRenderContexts within FSchemaHandlerRegistry::Register.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Translator")
	TArray<FString> CustomRenderContexts;
};
