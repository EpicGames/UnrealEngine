// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandlers/SchemaHandlerEntry.h"
#include "SchemaHandlers/SchemaHandler.h"
#include "SchemaHandlers/MaterialSchemaHandler.h"

#include "Containers/AnsiString.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

namespace UE::Interchange::USD
{
	/** Main lambda invoked when spawning an FSchemaHandler */
	using FSchemaHandlerGenerator = TFunction<TSharedRef<FSchemaHandler>(const FSchemaHandlerEntry& Entry)>;

	/**
	 * Static storage of all registered FSchemaHandlers.
	 */
	class FSchemaHandlerRegistry
	{
	public:
		/**
		 * Variable that stores the actual registered schema handlers.
		 * This is intentionally exposed so as to allow free insertion/reorder/removal of registered handlers.
		 */
		static UE_API TArray<FSchemaHandlerEntry> RegisteredHandlerEntries;

		/**
		 * Separate mapping of names to generators. This is useful because it lets us freely copy and serialize
		 * our handler entries to reorder / configure them.
		 *
		 * Registering your handler via FSchemaHandlerRegistry::Register() will also register it here.
		 *
		 * Whenever a handler is executed without having a corresponding generator function registered for its name,
		 * a warning will be emitted and the handler will be skipped.
		 */
		static UE_API TMap<FString, FSchemaHandlerGenerator> RegisteredHandlerNamesToGenerators;

	public:
		/**
		 * Registers a schema handler of a particular type, so that it is invoked when translating USD files.
		 * Usage:
		 *     FString HandlerName = FSchemaHandlerRegistry::Register<FImageableSchemaHandler>();
		 */
		template<typename SchemaHandlerType>
		static FString Register()
		{
			SchemaHandlerType TempHandler{};
			const FString& HandlerName = TempHandler.GetHandlerName();

			FSchemaHandlerGenerator GeneratorFunc;

			FSchemaHandlerEntry& NewEntry = RegisteredHandlerEntries.Emplace_GetRef();
			NewEntry.HandlerName = HandlerName;
			NewEntry.bEnabled = TempHandler.IsEnabled();
			NewEntry.SchemaName = TempHandler.GetTargetSchemaName();
			if constexpr (std::is_base_of_v<FMaterialSchemaHandler, SchemaHandlerType>)
			{
				FMaterialSchemaHandler* MaterialHandler = static_cast<FMaterialSchemaHandler*>(&TempHandler);
				NewEntry.DefaultRenderContexts = MaterialHandler->GetDefaultRenderContexts();
				NewEntry.bAllowCustomRenderContexts = MaterialHandler->AllowCustomRenderContexts();
				if (MaterialHandler->AllowCustomRenderContexts())
				{
					// Initialize our custom render contexts with the same value as the default
					NewEntry.CustomRenderContexts = NewEntry.DefaultRenderContexts;
				}

				GeneratorFunc = [](const FSchemaHandlerEntry& Entry)
				{
					TSharedRef<FSchemaHandler> Result = MakeShared<SchemaHandlerType>();
					Result->SetEnabled(Entry.bEnabled);

					TSharedRef<FMaterialSchemaHandler> MaterialHandlerResult = StaticCastSharedRef<FMaterialSchemaHandler>(Result);
					if (MaterialHandlerResult->AllowCustomRenderContexts())
					{
						MaterialHandlerResult->SetCustomRenderContexts(Entry.CustomRenderContexts);
					}

					return Result;
				};
			}
			else
			{
				GeneratorFunc = [](const FSchemaHandlerEntry& Entry)
				{
					TSharedRef<FSchemaHandler> Result = MakeShared<SchemaHandlerType>();
					Result->SetEnabled(Entry.bEnabled);
					return Result;
				};
			}

			RegisteredHandlerNamesToGenerators.Add(HandlerName, GeneratorFunc);

			return HandlerName;
		}

		/**
		 * Unregisters a schema handler, so that it will no longer be invoked when translating USD files.
		 * Returns true if it unregistered anything.
		 */
		static UE_API bool Unregister(const FString& HandlerName);

		/**
		 * Spawns an instance of each entry in RegisteredHandlerEntries, in order.
		 *
		 * By default this will use the registered handler entries, but a different array of entries can be provided.
		 *
		 * Before translation, the UInterchangeUSDTranslator calls this function and retains those same spawned handlers
		 * throughout the entire import process, so they are allowed to keep internal state between translation and
		 * payload retrieval
		 */
		static UE_API TArray<TSharedRef<FSchemaHandler>> GenerateHandlers(const TArray<FSchemaHandlerEntry>* Entries = &RegisteredHandlerEntries);
	};
}	 // namespace UE::Interchange::USD

#undef UE_API

#endif	  // USE_USD_SDK
