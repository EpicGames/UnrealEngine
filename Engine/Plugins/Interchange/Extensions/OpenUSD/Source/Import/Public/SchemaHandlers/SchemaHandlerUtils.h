// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "InterchangeUsdTraversalInfo.h"
#include "USDConversionUtils.h"
#include "USDLog.h"
#include "UsdWrappers/UsdPrim.h"

#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeAnimationDefinitions.h"
#include "InterchangeAnimationTrackSetNode.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeMeshNode;
class UInterchangeSceneNode;
class UInterchangeUsdContext;
enum class EInterchangeMeshCollision : uint8;
enum TextureGroup : int;
struct FMeshDescription;
namespace UE
{
	namespace Interchange
	{
		struct FAnimationPayloadQuery;
		struct FAnimationPayloadData;
		struct FNaniteAssemblyDescription;
		namespace USD
		{
			class FHandlerAccumulatedInfo;
		}
	}
}
namespace UsdToUnreal
{
	struct FTextureParameterValue;
	struct FUsdMeshConversionOptions;
}
namespace UsdUtils
{
	struct FUsdPrimMaterialSlot;
	struct FUsdPrimMaterialAssignmentInfo;
}

namespace UE::Interchange::USD
{
	/**
	* Generates a case sensitive hash for NodeUid
	*/
	UE_API FString GenerateHash(const FString& NodeUid);

	/**
	 * Append a case sensistive hash of NodeUid to the NodeUid itself, making it so that any TMap<FString, T> that stores
	 * these IDs behaves in a case sensitive manner.
	 *
	 * This is needed because unfortunately the default TMap<FString, T> is not case sensitive on the key FStrings, while prim
	 * names are case sensitive. Even if we could modify the UInterchangeBaseNodeContainer::Nodes map to be case sensitive, we'd
	 * still constantly get issues as any other TMap<FString, T> in the codebase that stores NodeUids would show collisions
	 */
	UE_API FString MakeNodeUid(const FString& NodeUid);

	/**
	* Removes the Hash postfix and '\\Mesh\\' prefix if present.
	* (Hash Postfix from MakeNodeUid).
	*/
	UE_API FString GetMeshPrimPathFromNodeUid(const FString& NodeUid);


	/**
	 * Produces a node UID intended to be used by interchange scene nodes that represent USD bones. This also internally uses calls MakeNodeUid().
	 * @param SkeletonPrimPath - Full path to the skeleton prim (e.g. /World/Character/Skel)
	 * @param ConcatBonePath - Concatenated path to the current bone (e.g. Hips/Neck/Shoulder)
	 * @return The full node uid for the bone (e.g. "\Bone\/World/Character/Skel/Hips/Neck/shoulder_46271545415")
	 */
	UE_API FString MakeBoneNodeUid(const FString& SkeletonPrimPath, const FString& ConcatBonePath);

	/**
	 * Produces a node UID intended to be used by the scene node that represents a root bone of a skeleton (this may even be an artificial,
	 * extra bone added to the skeleton in case the USD skeleton contains multiple root bones). This also internally uses calls MakeNodeUid().
	 * Unlike MakeBoneNodeUid(), this will not actually include any bone name into the UID, and instead will use a standard "Root" suffix 
	 * (e.g. "\Bone\/World/Character/Skel/Root_46271545415").
	 * The intent is that this allows easy construction of the root joint UID from the skeleton prim path without having to parse the
	 * USD skeleton itself all the time.
	 */
	UE_API FString MakeRootBoneNodeUid(const FString& SkeletonPrimPath);

	/** Produces a node UID meant to be used by mesh nodes that represent USD blend shapes. Internally calls UsdContext.MakeAssetNodeUid(). */
	UE_API FString GetMorphTargetMeshNodeUid(
		const UInterchangeUsdContext& UsdContext,
		const UE::FUsdPrim& MeshPrim,
		int32 MeshBlendShapeIndex,
		const FString& InbetweenName = FString{}
	);
	UE_DEPRECATED(5.8, "Use the other signature that receives the UsdContext")
	UE_API FString GetMorphTargetMeshNodeUid(const FString& MeshPrimPath, int32 MeshBlendShapeIndex, const FString& InbetweenName = FString{});

	/**
	 * Utility to facilitate trying to fetch a node with a particular UID *and* a specific type from the NodeContainer,
	 * emitting a warning in case of an unexpected type.
	 */
	template<typename T>
	T* GetExistingNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUid)
	{
		T* Result = nullptr;
		if (UInterchangeBaseNode* ExistingNode = const_cast<UInterchangeBaseNode*>(NodeContainer.GetNode(NodeUid)))
		{
			Result = Cast<T>(ExistingNode);
			if (!Result)
			{
				UE_LOGF(
					LogUsd,
					Warning,
					"Tried to retrievie a '%ls' node with UID '%ls', but encountered incompatible node of class '%ls' with the same UID already in the node container! The node may not be translated correctly.",
					*T::StaticClass()->GetPathName(),
					*NodeUid,
					*ExistingNode->GetClass()->GetPathName()
				);
				Result = nullptr;
			}
		}

		return Result;
	}

	/**
	 * Creates a generic scene node corresponding to a prim, if that is appropriate.
	 * Also converts attributes and metadata into the new node, if needed.
	 */
	UE_API UInterchangeSceneNode* GetOrCreateDefaultSceneNode(const UE::FUsdPrim& Prim, const FTraversalInfo& Info, UInterchangeUsdContext& UsdContext);

	/*
	 * Generically converts a non-animated prim attribute into Interchange user attributes, adding them to Node.
	 */
	UE_API void TranslateAttributeAsUserAttribute(
		const UE::FUsdPrim& Prim,
		const FString& AttributeName,
		UInterchangeBaseNode* Node,
		const double TimeCode = UsdUtils::GetDefaultTimeCode(),
		bool bAuthoredOnly = true
	);

	/**
	 * Generically converts non-animated prim attributes into Interchange user attributes, adding them to Node.
	 * A regex can be provided and will be matched against the attribute's name to select which attributes to convert.
	 */
	UE_API void TranslateAttributes(const UE::FUsdPrim& Prim, UInterchangeBaseNode* Node, const FString& AllowedAttributeRegex);

	/**
	 * Retrieves all attribute names registered for the provided schema, and translates them all into Interchange user attributes on Node
	 */
	UE_API bool TranslateAPISchemaAttributes(const UE::FUsdPrim& Prim, const FString& AppliedAPISchemaName, UInterchangeBaseNode* Node);

	/**
	 * Generically converts prim metadata values into Interchange user attributes, adding them to Node.
	 * A regex can be provided and will be matched against the metadata field's name to select which fields to convert.
	 * Nested metadata dictionaries will be flattened, using ':' as a separator
	 */
	UE_API void TranslateMetadata(const UE::FUsdPrim& Prim, UInterchangeBaseNode* Node, const FString& AllowedMetadataRegex);

	/**
	 * Creates a new UInterchangeTransformAnimationTrackNode node describing transform animations on Prim, and adds the node to
	 * accumulated info, the node container, and the current track set node.
	 */
	UE_API void AddTransformAnimationNode(const UE::FUsdPrim& Prim, FHandlerAccumulatedInfo& AccumulatedInfo, UInterchangeUsdContext& UsdContext);

	/**
	 * Creates a new UInterchangeAnimationTrackNode node describing animation of a provided UE property of the given SceneNodeUid,
	 * and adds the node to accumulated info, the node container, and the current track set node.
	 */
	UE_API void AddPropertyAnimationNode(
		const FString& SceneNodeUid,
		const FName& UEPropertyName,
		EInterchangePropertyTracks TrackType,
		EInterchangeAnimationPayLoadType PayloadType,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	);

	struct FInterchangeTrackInfo
	{
		FName PropertyName;
		EInterchangePropertyTracks TrackType;
		EInterchangeAnimationPayLoadType PayloadType = EInterchangeAnimationPayLoadType::CURVE;
	};

	/**
	 * Iterates through UsdAttributeNameToTrackInfo and gets the Prim's attributes matching the map's keys. Then, for each animated attribute,
	 * invokes AddPropertyAnimationNode() using the corresponding TrackInfo map value as arguments.
	 *
	 * Multiple FInterchangeTrackInfo can be provided because sometimes USD attributes map to multiple UE properties (e.g. if we animate
	 * sensorWidth on an USD orthographic camera, we need to animate both OrthoWidth and AspectRatio in UE)
	 */
	UE_API void AddNodesForAnimatedAttributes(
		const UE::FUsdPrim& Prim,
		const TMap<FString, TArray<FInterchangeTrackInfo>>& UsdAttributeNameToTrackInfo,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	);

	/**
	 * These functions are intended to be used when producing payloads for animation tracks of different types.
	 * ReaderFunc will be invoked at the provided UsdTimeSamples and collect the animated track key-value pairs into OutPayloadData.
	 * The produced keys contain time values in seconds, so a reference to the UsdStage is needed to provide the frame rate conversions.
	 * These functions return true if the payload was produced successfully.
	 */
	UE_API bool ReadBools(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<bool(double)>& ReaderFunc,
		UE::Interchange::FAnimationPayloadData& OutPayloadData
	);
	UE_API bool ReadFloats(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<float(double)>& ReaderFunc,
		UE::Interchange::FAnimationPayloadData& OutPayloadData
	);
	UE_API bool ReadColors(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FLinearColor(double)>& ReaderFunc,
		UE::Interchange::FAnimationPayloadData& OutPayloadData
	);
	UE_API bool ReadTransforms(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FTransform(double)>& ReaderFunc,
		UE::Interchange::FAnimationPayloadData& OutPayloadData
	);
	UE_API bool ReadRawTransforms(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FTransform(double)>& ReaderFunc,
		Interchange::FAnimationPayloadData& OutPayloadData
	);

	/**
	 * If MaterialNode describes a one-sided material, this will create (or reuse from the NodeContainer) a new material instance node that
	 * describes a two-sided version of the material.
	 *
	 * If the MaterialNode itself represents an instance of an UsdPreviewSurface parent material, the produced material will just be an
	 * instance of the two-sided version of that parent material. In other cases, the material instance node will have the provided one-sided
	 * material as its parent.
	 *
	 * This function may also just return MaterialNode in case it is already a two-sided material, or if there's nothing that can be done
	 * (e.g. in case MaterialNode is an UInterchangeMaterialReferenceNode)
	 */
	UE_API UInterchangeBaseNode* GetOrCreateTwoSidedMaterial(UInterchangeBaseNode* MaterialNode, UInterchangeBaseNodeContainer& NodeContainer);

	/**
	 * Iterates through MaterialAssignments, producing material nodes and setting them as slot material dependencies
	 * on MeshOrSceneNode via SetSlotMaterialDependencyUid()
	 *
	 * Note that this function may invoke new translations, for example when resolving a material assignment that points at another
	 * material prim, or when referencing a MaterialX material
	 */
	template<typename MeshOrSceneNodeType>
	UE_API void SetSlotMaterialDependencies(
		MeshOrSceneNodeType* MeshOrSceneNode,
		const UsdUtils::FUsdPrimMaterialAssignmentInfo& MaterialAssignments,
		FHandlerAccumulatedInfo& AccumulatedInfo,
		UInterchangeUsdContext& UsdContext
	);

	/**
	 * Fixup material slot names to match the material that is assigned. For Interchange it is better to have the material
	 * slot names match what is assigned into them, as it will use those names to "merge identical slots" depending on the
	 * import options.
	 *
	 * Note: These names must also match what is set via MeshNode->SetSlotMaterialDependencyUid(SlotName, MaterialUid)
	 */
	 UE_API void FixMaterialSlotNames(
		 FMeshDescription& MeshDescription,
		 const TArray<UsdUtils::FUsdPrimMaterialSlot>& MeshAssingmentSlots,
		 UInterchangeUsdContext& UsdContext
	);
	UE_DEPRECATED(5.8, "Use the other overload that receives a UsdContext");
	UE_API void FixMaterialSlotNames(FMeshDescription& MeshDescription, const TArray<UsdUtils::FUsdPrimMaterialSlot>& MeshAssingmentSlots);

	/**
	 * If we're not translating a decompressed USD root, returns TexturePathOnDisk.
	 * If we are translating a decompressed USD root, returns the path to the USDZ file itself.
	 *
	 * The intent here is that in the USDZ case the texture filepath will point at a temp file on disk, that we may dispose
	 * of later after importing. In order to allow reimporting the texture at a later time, we'll just put the USDZ path
	 * itself as it's source path, and tweak the USD translator to know what to do with this
	 */
	UE_API FString GetTextureSourcePath(const FString& TexturePathOnDisk);

	/**
	 * Checks whether the provided prim name corresponds to a LOD mesh prim name, in case of our convention of variant-set-based LODs.
	 * In other words, returns true when the prim name is of the form "LODN" or "LOD_N" where N is a number
	 */
	UE_API bool IsValidLODName(const FString& PrimName);

	/** If the provided name returns true for IsValidLODName(), the number N will be returned (see the documentation for IsValidLODName())*/
	UE_API int32 GetLODIndexFromName(FString Name);

	/**
	 * Given a prim that contains an LOD variant set, will return the Mesh prim that corresponds to the provided LOD prim name (e.g. LOD2)
	 */
	UE_API UE::FUsdPrim GetLODMesh(const UE::FUsdPrim& LODContainerPrim, const FString& LODName);
	
	/**
	* Contrary to GetLODMesh, this allows non mesh types.
	*/
	UE_API UE::FUsdPrim GetLODPrim(const UE::FUsdPrim& LODContainerPrim, const FString& LODName);

	/**
	 * When given a full path to a LOD mesh prim (e.g. /World/LODContainer/LOD2), this will return the LOD2 mesh prim. It will do this
	 * by actively switching the variant on the LODContainer by opening (or reusing) an entirely separate stage kept in the UsdContext,
	 * and returning the prim from there.
	 *
	 * You should only use this function if you can't find PrimPathString on the main UsdContext stage, due to it not belonging to the
	 * currently active variant
	 */
	UE_API UE::FUsdPrim TryGettingInactiveLODPrim(const FString& PrimPathString, UInterchangeUsdContext& UsdContext);

	/** If PayloadKey starts with Prefix, it removes it and returns true */
	UE_API bool CheckAndChopPayloadPrefix(FString& PayloadKey, const FString& Prefix);

	/** These functions encode or decode information into/from FStrings intended to be used as translated node payload keys */
	UE_API bool DecodeTexturePayloadKey(const FString& PayloadKey, FString& OutTextureFilePath, TextureGroup& OutTextureGroup);
	UE_API FString EncodeTexturePayloadKey(const UsdToUnreal::FTextureParameterValue& Value);
	UE_API FString GetMorphTargetCurvePayloadKey(const FString& SkeletonPrimPath, int32 SkelAnimChannelIndex, const FString& BlendShapePath);
	UE_API FString GetMorphTargetMeshPayloadKey(
		bool bIsInsideLOD,
		const FString& MeshPrimPath,
		int32 MeshBlendShapeIndex,
		const FString& InbetweenName = FString{}
	);
	UE_API bool ParseMorphTargetMeshPayloadKey(
		FString InPayloadKey,
		bool& bOutIsLODMesh,
		FString& OutMeshPrimPath,
		int32& OutBlendShapeIndex,
		FString& OutInbetweenName
	);

	/**
	 * This function produces an unique hash describing the FAnimationPayloadQuery, which is currently used to help group up
	 * multiple queries that could be resolved together (i.e. correspond to the same skeleton with the same bake settings)
	 */
	UE_API FString HashAnimPayloadQuery(const Interchange::FAnimationPayloadQuery& Query);

	/**
	 * These functions are used by schema handlers to produce payload data of a particular type from the provided prim.
	 * They are exposed here as they must be used by multiple schema handlers.
	 *
	 * For more information check IInterchangeMeshPayloadInterface and InterchangeMeshPayload.h
	 */
	UE_API void GetNaniteAssemblyPayloadDataForPrims(
		const UE::FUsdStage& UsdStage,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult& TraversalResult,
		const FString& SkelIdentifier,
		const TArray<FString>& JointNames,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription
	);
	UE_API void GetNaniteAssemblyPayloadDataForPointInstancer(
		const UE::FUsdPrim& PointInstancerPrim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const UsdToUnreal::NaniteAssemblyUtils::FNaniteAssemblyTraversalResult& TraversalResult,
		const FString& SkelIdentifier,
		const TArray<FString>& JointNames,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription
	);
	UE_API bool GetStaticMeshPayloadData(
		FString PayloadKey,
		UInterchangeUsdContext& UsdContext,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FMeshDescription& OutMeshDescription,
		TOptional<UE::Interchange::FNaniteAssemblyDescription>& OutNaniteAssemblyDescription
	);

	/**
	* Combines all LocalTransforms from Descendant up to the Ancestor.
	*/
	UE_API FTransform GetCombinedTransform(const UE::FUsdPrim& Ancestor, const UE::FUsdPrim& Descendant);

	/**
	* Sets/Stores the PrimPath on the BaseNode
	*/
	UE_API void SetPrimPath(UInterchangeBaseNode& BaseNode, const FString& PrimPath);

	/**
	* Queries the PrimPath from the BaseNode
	*/
	UE_API FString GetPrimPath(const UInterchangeBaseNode& BaseNode);

	/**
	* Queries the Mesh PrimPath from the BaseNode, if it does not exist it will try to deduce form the UniqueID.
	*/
	UE_API FString GetMeshPrimPath(const UInterchangeBaseNode& BaseNode);
}

#undef UE_API

#endif	  // USE_USD_SDK
