// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandlers/MaterialSchemaHandler.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

namespace UE::Interchange::USD
{
	class FUniversalMaterialSchemaHandler : public FMaterialSchemaHandler
	{
	public:
		UE_API virtual const FString& GetHandlerName() const override;

		UE_API const TArray<FString>& GetDefaultRenderContexts() const override;

		UE_API virtual bool AllowCustomRenderContexts() const override;

		UE_API virtual TOptional<bool> CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual bool OnTranslate(
			const UE::FUsdPrim& Prim,
			FTraversalInfo& TraversalInfo,
			FHandlerAccumulatedInfo& AccumulatedInfo,
			UInterchangeUsdContext& UsdContext
		) override;

		UE_API virtual bool OnGetTexturePayloadData(
			const FString& PayloadKey,
			TOptional<FString>& AlternateTexturePath,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FImportImage>& InOutPayloadData
		) override;

		UE_API virtual bool OnGetBlockedTexturePayloadData(
			const FString& PayloadKey,
			TOptional<FString>& AlternateTexturePath,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FImportBlockedImage>& InOutPayloadData
		) override;
	};
}

#undef UE_API

#endif	  // USE_USD_SDK