// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandler.h"

#include "USDConversionUtils.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

class UInterchangeVolumeNode;
namespace UsdUtils
{
	struct FVolumePrimInfo;
}

namespace UE::Interchange::USD
{
	class FVolumeSchemaHandler : public FSchemaHandler
	{
	public:
		UE_API virtual const FString& GetHandlerName() const override;

		UE_API virtual const FString& GetTargetSchemaName() const override;

		UE_API virtual TOptional<bool> CanBeCollapsed(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual TOptional<bool> CollapsesChildren(const UE::FUsdPrim& Prim, UInterchangeUsdContext& UsdContext) const override;

		UE_API virtual bool OnTranslate(
			const UE::FUsdPrim& Prim,
			FTraversalInfo& TraversalInfo,
			FHandlerAccumulatedInfo& AccumulatedInfo,
			UInterchangeUsdContext& UsdContext
		) override;

		UE_API virtual bool OnGetVolumePayloadData(
			const UE::Interchange::FVolumePayloadKey& PayloadKey,
			UInterchangeUsdContext& UsdContext,
			TOptional<UE::Interchange::FVolumePayloadData>& InOutPayloadData
		) override;

		UE_API virtual bool OnGetAnimationPayloadData(
			const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
			UInterchangeUsdContext& UsdContext,
			TArray<UE::Interchange::FAnimationPayloadData>& InOutPayloadData
		) override;

	private:
		// We stash the info we collected from each Volume prim path here, as we'll reuse it between translation and retrieving the payloads
		TMap<FString, TArray<UsdUtils::FVolumePrimInfo>> PrimPathToVolumeInfo;

		// Used within a translation. We cache these because we make a volume node *per .vdb file*, and on the USD side we may have
		// any number of Volume prims internally using the same .vdb file, and we want to share these whenever possible
		TMap<FString, TMap<FString, UInterchangeVolumeNode*>> VolumeFilepathToAnimationIDToNode;
	};
}

#undef UE_API

#endif	  // USE_USD_SDK