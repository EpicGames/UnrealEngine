// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "SchemaHandler.h"

#define UE_API INTERCHANGEOPENUSDIMPORT_API

namespace UE::Interchange::USD
{
	class FXformableSchemaHandler : public FSchemaHandler
	{
	public:
		UE_API virtual const FString& GetHandlerName() const override;

		UE_API virtual const FString& GetTargetSchemaName() const override;

		UE_API virtual bool OnTranslate(
			const UE::FUsdPrim& Prim,
			FTraversalInfo& TraversalInfo,
			FHandlerAccumulatedInfo& AccumulatedInfo,
			UInterchangeUsdContext& UsdContext
		) override;
	};
}

#undef UE_API

#endif	  // USE_USD_SDK